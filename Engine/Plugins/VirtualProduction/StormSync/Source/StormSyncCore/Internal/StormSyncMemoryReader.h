// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/UnrealMemory.h"
#include "Serialization/MemoryArchive.h"
#include "StormSyncCommonTypes.h"

/**
 * Archive for reading arbitrary data from a shared storm sync buffer.
 */
class FStormSyncMemoryReader : public FMemoryArchive
{
public:
	//~ Begin FArchive
	virtual FString GetArchiveName() const override
	{
		return TEXT("FStormSyncMemoryReader");
	}

	virtual int64 TotalSize() override
	{
		return Buffer ? Buffer->Num() : 0;
	}

	virtual void Serialize(void* OutData, int64 InNum) override
	{
		if (InNum && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Buffer && Offset + InNum <= TotalSize())
			{
				FMemory::Memcpy(OutData, &(*Buffer)[Offset], InNum);
				Offset += InNum;
			}
			else
			{
				SetError();
			}
		}
	}
	//~ End FArchive

	explicit FStormSyncMemoryReader(const FStormSyncBufferPtr& InBuffer)
		: Buffer(InBuffer)
	{
		SetIsLoading(true);
		SetIsPersistent(false);
	}

private:
	/** Keep a shared reference on the buffer. */
	FStormSyncBufferPtr Buffer;
};