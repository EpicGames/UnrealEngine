// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncCommandLineUtils.h"

#include "Commandlets/Commandlet.h"

void FStormSyncCommandLineUtils::Parse(const TCHAR* CmdLine, TArray<FName>& Arguments)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> Params;
	UCommandlet::ParseCommandLine(CmdLine, Tokens, Switches, Params);
	
	for (FString Token : Tokens)
	{
		Arguments.Add(FName(*Token));
	}
}
