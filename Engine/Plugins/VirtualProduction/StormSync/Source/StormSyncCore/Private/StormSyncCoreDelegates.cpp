// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncCoreDelegates.h"

FStormSyncCoreDelegates::FOnRequestImportBuffer FStormSyncCoreDelegates::OnRequestImportBuffer;
FStormSyncCoreDelegates::FOnRequestImportFile FStormSyncCoreDelegates::OnRequestImportFile;
FStormSyncCoreDelegates::FOnFileImported FStormSyncCoreDelegates::OnFileImported;
FStormSyncCoreDelegates::FOnPakAssetExtracted FStormSyncCoreDelegates::OnPakAssetExtracted;
FStormSyncCoreDelegates::FOnPreStartSendingBuffer FStormSyncCoreDelegates::OnPreStartSendingBuffer;
FStormSyncCoreDelegates::FOnStartSendingBuffer FStormSyncCoreDelegates::OnStartSendingBuffer;
FStormSyncCoreDelegates::FOnReceivingBytes FStormSyncCoreDelegates::OnReceivingBytes;
FStormSyncCoreDelegates::FOnTransferComplete FStormSyncCoreDelegates::OnTransferComplete;
FStormSyncCoreDelegates::FOnServiceDiscoveryConnection FStormSyncCoreDelegates::OnServiceDiscoveryConnection;
FStormSyncCoreDelegates::FOnServiceDiscoveryStateChange FStormSyncCoreDelegates::OnServiceDiscoveryStateChange;
FStormSyncCoreDelegates::FOnServiceDiscoveryReceivedWakeup FStormSyncCoreDelegates::OnServiceDiscoveryReceivedWakeup;
FStormSyncCoreDelegates::FOnServiceDiscoveryServerStatusChange FStormSyncCoreDelegates::OnServiceDiscoveryServerStatusChange;
FStormSyncCoreDelegates::FOnServiceDiscoveryDisconnection FStormSyncCoreDelegates::OnServiceDiscoveryDisconnection;
FStormSyncCoreDelegates::FOnStormSyncServerStarted FStormSyncCoreDelegates::OnStormSyncServerStarted;
FStormSyncCoreDelegates::FOnStormSyncServerStopped FStormSyncCoreDelegates::OnStormSyncServerStopped;
