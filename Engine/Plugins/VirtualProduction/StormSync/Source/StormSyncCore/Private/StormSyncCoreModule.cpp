// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "StormSyncCoreLog.h"

DEFINE_LOG_CATEGORY(LogStormSyncCore);

IMPLEMENT_MODULE(FDefaultModuleImpl, StormSyncCore)
