// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncCoreSettings.h"

UStormSyncCoreSettings::UStormSyncCoreSettings()
{
	CategoryName = TEXT("Storm Sync");

	IgnoredPackagesInternal.Add(TEXT("/Engine"));
	IgnoredPackagesInternal.Add(TEXT("/Script"));
	ExportDefaultNameFormatString = TEXT("%Y_%m_%d_%H%M%S");
}
