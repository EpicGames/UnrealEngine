// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncCoreUtils.h"

#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncCoreLog.h"
#include "StormSyncCoreSettings.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "StormSyncCoreUtils"

namespace UE::StormSyncCore::Private
{
	// For now, we use the same buffer size as the file writer.
	// Todo: It might be beneficial to have an implementation for uncached reads/writes
	// for large file operations which may benefit from buffers that are more aligned with
	// physical device's page size.
	constexpr uint64 ChunkBufferSize = PLATFORM_FILE_WRITER_BUFFER_SIZE;

	constexpr uint64 GetChunkBufferSize()
	{
		return ChunkBufferSize;
	}

	/** Utility function to copy from one archive to another using a temporary buffer. */
	void ArchiveCopy(FArchive& InReadArchive, FArchive& InWriteArchive, int64 InSizeToCopy, TArray64<uint8>& InChunkBuffer)
	{
		// Note: This should already be validated prior, but we ensure here as a precaution.
		if (!ensure(InReadArchive.IsLoading()) || !ensure(InWriteArchive.IsSaving()))
		{
			return;
		}
		
		while (InSizeToCopy > 0)
		{
			const int64 ChunkSize = FMath::Min(InChunkBuffer.Num(), InSizeToCopy);
			InReadArchive.Serialize(InChunkBuffer.GetData(), ChunkSize);
			InWriteArchive.Serialize(InChunkBuffer.GetData(), ChunkSize);
			InSizeToCopy -= ChunkSize;
		}
	}
}

bool FStormSyncCoreUtils::GetAssetData(const FString& InPackageName, TArray<FAssetData>& OutAssets, TArray<FName>& OutDependencies)
{
	FString Filename = InPackageName;

	// Get the filename by finding it on disk first
	if (!FPackageName::DoesPackageExist(InPackageName, &Filename))
	{
		// The package does not exist on disk, see if we can find it in memory and predict the file extension
		// Only do this if the supplied package name is valid
		constexpr bool bIncludeReadOnlyRoots = false;
		if (FPackageName::IsValidLongPackageName(InPackageName, bIncludeReadOnlyRoots))
		{
			const UPackage* Package = FindPackage(nullptr, *InPackageName);
			// This is a package in memory that has not yet been saved. Determine the extension and convert to a filename, if we do have the package, just assume normal asset extension
			const FString PackageExtension = Package && Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
			Filename = FPackageName::LongPackageNameToFilename(InPackageName, PackageExtension);
		}
	}

	const FString AbsoluteFilename = FPaths::ConvertRelativePathToFull(Filename);

	// Filter on improbable file extensions
	const EPackageExtension PackageExtension = FPackagePath::ParseExtension(AbsoluteFilename);

	if (PackageExtension == EPackageExtension::Unspecified || PackageExtension == EPackageExtension::Custom)
	{
		return false;
	}

	constexpr bool bGetDependencies = true;
	IAssetRegistry::FLoadPackageRegistryData LoadedData(bGetDependencies);

	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.LoadPackageRegistryData(AbsoluteFilename, LoadedData);

	OutAssets = MoveTemp(LoadedData.Data);

	for (const FName& Dependency : LoadedData.DataDependencies)
	{
		// Prevent adding twice same dependency
		if (OutDependencies.Contains(Dependency))
		{
			continue;
		}
		
		// Exclude script/memory packages
		if (FPackageName::IsValidLongPackageName(Dependency.ToString()))
		{
			OutDependencies.Add(Dependency);
		}
	}

	// Add the original package name as part of the dependency response
	OutDependencies.Add(FName(*InPackageName));
	return !OutAssets.IsEmpty();
}

bool FStormSyncCoreUtils::GetDependenciesForPackages(const TArray<FName>& InPackageNames, TArray<FName>& OutDependencies, FText& OutErrorText, const bool bInShouldValidatePackages)
{
	if (InPackageNames.IsEmpty())
	{
		OutErrorText = LOCTEXT("PackageNames_Empty", "Provided PackageNames array is empty.");
		return false;
	}

	// Validate provided files upfront, checking it is a valid reference path
	if (bInShouldValidatePackages && !ValidateAssets(InPackageNames, OutErrorText))
	{
		return false;
	}

	TArray<FString> PackageNames;
	Algo::Transform(InPackageNames, PackageNames, [] (const FName& PackageName) { return PackageName.ToString(); });
	
	// Gather all dependencies for provided files
	for (const FName& PackageName : InPackageNames)
	{
		// Make sure to also include the entry itself
		OutDependencies.Add(PackageName);
		RecursiveGetDependencies(PackageName, OutDependencies);
	}

	// Right now, simply sort out file alphabetically.
	// engine's PakFileUtilities also has a SuggestedOrder mechanism we might want to implement as well
	OutDependencies.Sort(FNameLexicalLess());

	return true;
}

bool FStormSyncCoreUtils::GetAvaFileDependenciesForPackages(const TArray<FName>& InPackageNames, TArray<FStormSyncFileDependency>& OutFileDependencies, FText& OutErrorText, const bool bInShouldValidatePackages)
{
	TArray<FName> PackageDependencies;
	if (!GetDependenciesForPackages(InPackageNames, PackageDependencies, OutErrorText, bInShouldValidatePackages))
	{
		return false;
	}

	const TArray<FStormSyncFileDependency> FileDependencies = GetAvaFileDependenciesFromPackageNames(PackageDependencies, !bInShouldValidatePackages);
	OutFileDependencies.Append(FileDependencies);

	return true;
}

TArray<FStormSyncFileDependency> FStormSyncCoreUtils::GetAvaFileDependenciesFromPackageNames(const TArray<FName>& InPackageNames, bool bInShouldIncludeInvalid)
{
	TArray<FStormSyncFileDependency> FileDependencies;
	for (const FName& PackageName : InPackageNames)
	{
		FStormSyncFileDependency FileDependency = CreateStormSyncFile(PackageName);
		if (FileDependency.IsValid() || bInShouldIncludeInvalid)
		{
			FileDependencies.Add(FileDependency);
		}
	}

	return FileDependencies;
}

TFuture<TArray<FStormSyncFileDependency>> FStormSyncCoreUtils::GetAvaFileDependenciesAsync(const TArray<FName>& InPackageNames, const bool bInShouldValidatePackages, EAsyncExecution InThreadType)
{
	TArray<FName> LocalPackageNames = InPackageNames;
	return Async(InThreadType, [PackageNames = MoveTemp(LocalPackageNames), bInShouldValidatePackages]()
	{
		FText ErrorText;
		TArray<FStormSyncFileDependency> FileDependencies;

		if (!GetAvaFileDependenciesForPackages(PackageNames, FileDependencies, ErrorText, bInShouldValidatePackages))
		{
			UE_LOGF(LogStormSyncCore, Error, "FStormSyncCoreUtils::GetAvaFileDependenciesAsync - Error: %ls", *ErrorText.ToString());
			return FileDependencies;
		}

		return FileDependencies;
	});
}

FStormSyncFileDependency FStormSyncCoreUtils::CreateStormSyncFile(const FName& InPackageName)
{
	const FString PackageNameStr = InPackageName.ToString();

	FStormSyncFileDependency FileDependency(InPackageName);

	FString PackageFilepath;
	if (FPackageName::DoesPackageExist(PackageNameStr, &PackageFilepath))
	{
		if (const TUniquePtr<FArchive> FileHandle = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*PackageFilepath)))
		{
			FileDependency.Timestamp = IFileManager::Get().GetTimeStamp(*PackageFilepath).ToUnixTimestamp();
			FileDependency.FileSize = FileHandle->TotalSize();

			// Note: Consider another hashing algorithm (as per Matt's suggestion)
			const FMD5Hash FileMD5Hash = FMD5Hash::HashFile(*PackageFilepath);
			FileDependency.FileHash = LexToString(FileMD5Hash);

			// Close the file
			FileHandle->Close();
		}
	}

	return FileDependency;
}

bool FStormSyncCoreUtils::CreatePakBufferWithDependencies(const TArray<FName>& InPackageNames, FArchive& OutPakArchive, FText& OutErrorText, const FOnFileAdded& InOnFileAdded)
{
	TArray<FName> PackageDependencies;
	if (!GetDependenciesForPackages(InPackageNames, PackageDependencies, OutErrorText))
	{
		return false;
	}

	return CreatePakBuffer(PackageDependencies, OutPakArchive, OutErrorText, InOnFileAdded);
}

bool FStormSyncCoreUtils::CreatePakBuffer(const TArray<FName>& InPackageNames, FArchive& OutPakArchive, FText& OutErrorText, const FOnFileAdded& InOnFileAdded)
{
	const double StartTime = FPlatformTime::Seconds();

	// Validate we have some files to add before doing anything
	if (InPackageNames.IsEmpty())
	{
		OutErrorText = LOCTEXT("CreatePakBuffer_PackageNames_Empty", "Provided PackageNames array is empty.");
		return false;
	}

	if (!OutPakArchive.IsSaving())
	{
		OutErrorText = FText::Format(
			LOCTEXT("CreatePakBuffer_ArchiveNotWritable", "CreatePakBuffer: Archive \"{0}\" is not writable."),
			FText::FromString(OutPakArchive.GetArchiveName()));
		return false;
	}

	TArray<FName> PackageNames;

	const UStormSyncCoreSettings* Settings = GetDefault<UStormSyncCoreSettings>();
	check(Settings);

	// If user opted to always ignore non existing file on disk
	if (Settings->bFilterInvalidReferences)
	{
		// Just filter them out and ignore. This is very low level, if caller wants to handle that case and
		// maybe display invalid references as errors or warnings, it is expected to be done by caller before
		// calling this method
		PackageNames = InPackageNames.FilterByPredicate([](const FName& PackageName)
		{
			return FPackageName::DoesPackageExist(PackageName.ToString());
		});
	}
	else
	{
		// If not, just take the full list of incoming files
		PackageNames = InPackageNames;
	}

	// Validate provided files upfront, checking if all exists on disk
	if (!ValidateAssets(PackageNames, OutErrorText))
	{
		return false;
	}

	const int64 StartOffset = OutPakArchive.Tell();
	
	int32 FileCount = PackageNames.Num();
	OutPakArchive << FileCount;

	TArray64<uint8> ChunkBuffer;
	ChunkBuffer.SetNumUninitialized(UE::StormSyncCore::Private::GetChunkBufferSize());

	UE_LOGF(LogStormSyncCore, Display, "FStormSyncCoreUtils::CreatePakBuffer - Creating Pak file for %d files.", PackageNames.Num());
	
	for (const FName& PackageName : PackageNames)
	{
		const FString PackageNameStr = PackageName.ToString();
		UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::CreatePakBuffer - Handle `%ls` file to add.", *PackageNameStr);

		FString PackageFilepath;
		if (!FPackageName::DoesPackageExist(PackageNameStr, &PackageFilepath))
		{
			checkf(false, TEXT("Attempting to create pak with \"%s\" which is not a file."), *PackageNameStr);
			return false;
		}

		if (const TUniquePtr<FArchive> FileHandle = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*PackageFilepath)))
		{
			int64 FileSize = FileHandle->TotalSize();
			int64 Timestamp = IFileManager::Get().GetTimeStamp(*PackageFilepath).ToUnixTimestamp();

			// Note: Consider another hashing algorithm (as per Matt's suggestion)
			FMD5Hash FileMD5Hash = FMD5Hash::HashFile(*PackageFilepath);
			FString FileHash = LexToString(FileMD5Hash);

			// Write Package path
			FString FinalAssetName = PackageNameStr + FPaths::GetExtension(PackageFilepath, /*bIncludeDot*/true);
			OutPakArchive << FinalAssetName;

			// Write size of the buffer
			OutPakArchive << FileSize;

			// Write file timestamp
			OutPakArchive << Timestamp;

			// Write file hash
			OutPakArchive << FileHash;

			// Copy file content to pack
			UE::StormSyncCore::Private::ArchiveCopy(*FileHandle, OutPakArchive, FileSize, ChunkBuffer);

			// Close the file
			FileHandle->Close();

			InOnFileAdded.ExecuteIfBound(FStormSyncFileDependency(PackageName, FileSize, Timestamp, FileHash));
		}
	}

	UE_LOGF(LogStormSyncCore, Display, "Added %d files to %ls, %lld bytes total, time %.2lfs.", FileCount, *OutPakArchive.GetArchiveName(), OutPakArchive.Tell() - StartOffset, FPlatformTime::Seconds() - StartTime);

	return true;
}

bool FStormSyncCoreUtils::ExtractPakBuffer(FArchive& InPakArchive, const FStormSyncCoreExtractArgs& InExtractArgs, TArray<FText>& OutErrors)
{
	if (!InPakArchive.IsLoading())
	{
		UE_LOGF(LogStormSyncCore, Error, "FStormSyncCoreUtils:ExtractPakBuffer Cannot Extract package from non-reader archive \"%ls\"", *InPakArchive.GetArchiveName());
		return false;
	}
	
	UE_LOGF(LogStormSyncCore, Display, "FStormSyncCoreUtils:ExtractPakBuffer Extracting package from archive of size: %lld", InPakArchive.TotalSize());

	bool bSuccess = true;

	int32 FileCount = 0;
	InPakArchive << FileCount;

	// Notify pack extraction process is starting
	InExtractArgs.OnPakPreExtract.ExecuteIfBound(FileCount);

	UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils:ExtractPakBuffer FileCount: %d", FileCount);

	TArray64<uint8> ChunkBuffer;
	ChunkBuffer.SetNumUninitialized(UE::StormSyncCore::Private::GetChunkBufferSize());

	for (int32 Index = 0; Index < FileCount; ++Index)
	{
		// Extract package path
		FString PackagePath;
		InPakArchive << PackagePath;

		// We want original PackageName (as right now, pak creation stores the extension in pak file)
		const FString Extension = FPaths::GetExtension(PackagePath, /*bIncludeDot*/true);
		PackagePath.RemoveFromEnd(Extension);

		// Extract file Size
		uint64 FileSize = 0;
		InPakArchive << FileSize;

		// Extract file timestamp
		int64 Timestamp = 0;
		InPakArchive << Timestamp;

		// Extract file hash
		FString FileHash;
		InPakArchive << FileHash;

		// Figure out file destination on disk
		const FName PackageName = FName(*PackagePath);

		FText ErrorText;
		FString DestFilepath = FStormSyncFileDependency::GetDestFilepath(PackageName, ErrorText);
		if (DestFilepath.IsEmpty())
		{
			// We were not able to determine destination output, mark as errored.
			OutErrors.Add(ErrorText);
			bSuccess = false;
			InPakArchive.Seek(InPakArchive.Tell() + FileSize);	// Skip file content.
		}
		else
		{
			DestFilepath += Extension;

			// Notify pack extraction for individual files
			FStormSyncFileDependency FileDependency = FStormSyncFileDependency(PackageName, FileSize, Timestamp, FileHash);

			if (InExtractArgs.OnGetArchiveForExtract.IsBound())
			{
				const FStormSyncArchivePtr Archive = InExtractArgs.OnGetArchiveForExtract.Execute(FileDependency, DestFilepath);
				if (Archive && Archive->IsSaving())
				{
					UE::StormSyncCore::Private::ArchiveCopy(InPakArchive, *Archive, FileSize, ChunkBuffer);

					// Call the end of extraction with the archive.
					InExtractArgs.OnArchiveExtracted.ExecuteIfBound(FileDependency, DestFilepath, Archive);
				}
				else
				{
					if (Archive && !Archive->IsSaving())
					{
						const FString ErrorMessage = FString::Printf(TEXT("FStormSyncCoreUtils::ExtractPakBuffer - A non-saving archive was provided, skipping \"%s\"."), *DestFilepath); 
						OutErrors.Add(FText::FromString(ErrorMessage));
						bSuccess = false;
					}
					InPakArchive.Seek(InPakArchive.Tell() + FileSize);	// Skip file content.
				}
			}
			else if (InExtractArgs.OnFileExtract.IsBound())
			{
				// Load remaining data (raw buffer of the file itself)
				FStormSyncBufferPtr Buffer = MakeShared<FStormSyncBuffer>();
				Buffer->Reset(FileSize);
				Buffer->AddUninitialized(FileSize);
				InPakArchive.Serialize(Buffer->GetData(), Buffer->Num());
				InExtractArgs.OnFileExtract.Execute(FileDependency, DestFilepath, Buffer);
			}
			else
			{
				InPakArchive.Seek(InPakArchive.Tell() + FileSize);	// Skip file content.
			}
		}
	}

	// Notify pack extraction process is done
	InExtractArgs.OnPakPostExtract.ExecuteIfBound(FileCount);

	return bSuccess;
}

FString FStormSyncCoreUtils::GetHumanReadableByteSize(uint64 InSize)
{
	TArray<FString> Suffixes = {TEXT("B"), TEXT("KB"), TEXT("MB"), TEXT("GB")};

	int32 i = 0;
	double Bytes = InSize;

	constexpr uint16 Divider = 1024;
	constexpr float FloatDivider = 1024.0;

	const int32 SuffixesCount = Suffixes.Num();

	if (InSize > Divider)
	{
		for (i = 0; (InSize / Divider) > 0 && i < SuffixesCount - 1; i++, InSize /= Divider)
		{
			Bytes = InSize / FloatDivider;
		}
	}

	return FString::Printf(TEXT("%.02lf %s"), Bytes, *Suffixes[i]);
}

TArray<FStormSyncFileModifierInfo> FStormSyncCoreUtils::GetSyncFileModifiers(const TArray<FName>& InPackageNames, const TArray<FStormSyncFileDependency>& InRemoteDependencies)
{
	UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::GetSyncFileModifiers - InPackageNames: %d, InRemoteDependencies: %d", InPackageNames.Num(), InRemoteDependencies.Num());

	// If provided package names is empty, early out
	if (InPackageNames.IsEmpty())
	{
		return {};
	}

	// Compute now the list of modifiers
	TArray<FStormSyncFileModifierInfo> Modifiers;

	// Build up list of local dependencies
	FText ErrorText;
	TArray<FStormSyncFileDependency> LocalDependencies;
	// Silently fail here, we may be requested to check against top lvl package names that don't exist locally
	GetAvaFileDependenciesForPackages(InPackageNames, LocalDependencies, ErrorText);

	// First check based on local files, to catch any missing files on remote
	for (const FStormSyncFileDependency& LocalDependency : LocalDependencies)
	{
		const FStormSyncFileDependency* MatchingDependency = InRemoteDependencies.FindByPredicate([LocalDependency](const FStormSyncFileDependency& Item)
		{
			return LocalDependency.PackageName == Item.PackageName;
		});

		// Check for missing file on remote, meaning we are referencing the file but sender is not
		// It doesn't mean sender doesn't have the file though ...
		if (!MatchingDependency)
		{
			FStormSyncFileModifierInfo Modifier;
			Modifier.ModifierOperation = EStormSyncModifierOperation::Missing;
			Modifier.FileDependency = LocalDependency;
			UE_LOGF(LogStormSyncCore, Verbose, "\tAdding Modifier: %ls", *Modifier.ToString());
			Modifiers.Add(Modifier);
		}
	}

	// Convert list of remote dependencies to just their package names to gather their local state
	TArray<FName> RemotePackageNames;
	Algo::Transform(InRemoteDependencies, RemotePackageNames, [](const FStormSyncFileDependency& RemoteDependency)
	{
		return RemoteDependency.PackageName;
	});

	TArray<FStormSyncFileDependency> LocalFiles = GetAvaFileDependenciesFromPackageNames(RemotePackageNames);

	// Then check based on remote files, to catch any mismatched state (either missing locally, or present but with mismatched size and / or file hash)
	for (const FStormSyncFileDependency& RemoteDependency : InRemoteDependencies)
	{
		const FStormSyncFileDependency* MatchingDependency = LocalFiles.FindByPredicate([RemoteDependency](const FStormSyncFileDependency& Item)
		{
			return RemoteDependency.PackageName == Item.PackageName;
		});

		// File not present locally in the local dependencies, this is a an addition
		if (!MatchingDependency)
		{
			FStormSyncFileModifierInfo Modifier;
			Modifier.ModifierOperation = EStormSyncModifierOperation::Addition;
			Modifier.FileDependency = RemoteDependency;
			UE_LOGF(LogStormSyncCore, Verbose, "\tAdding Modifier: %ls", *Modifier.ToString());
			Modifiers.Add(Modifier);

			continue;
		}

		// File present locally, figure out if it's dirty

		bool bIsDirty = false;

		// Note: We can't really test right now against file timestamp, as when received from a pak, the file is created again locally, which will modify the timestamp
		// So right now, we only check for dirty state against file size and hash.

		// Check against FileSize
		if (MatchingDependency->FileSize != RemoteDependency.FileSize)
		{
			UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::GetSyncFileModifiers - Handle %ls", *RemoteDependency.PackageName.ToString());
			UE_LOGF(LogStormSyncCore, Verbose, "\tLocal: %ls", *MatchingDependency->ToString());
			UE_LOGF(LogStormSyncCore, Verbose, "\tRemote: %ls", *RemoteDependency.ToString());
			UE_LOGF(LogStormSyncCore, Verbose, "\tDirty because of mismatch filesize %lld vs %lld", MatchingDependency->FileSize, RemoteDependency.FileSize);
			bIsDirty = true;
		}
		// Check against File Hash
		else if (MatchingDependency->FileHash != RemoteDependency.FileHash)
		{
			UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::GetSyncFileModifiers - Handle %ls", *RemoteDependency.PackageName.ToString());
			UE_LOGF(LogStormSyncCore, Verbose, "\tLocal: %ls", *MatchingDependency->ToString());
			UE_LOGF(LogStormSyncCore, Verbose, "\tRemote: %ls", *RemoteDependency.ToString());
			UE_LOGF(LogStormSyncCore, Verbose, "\tDirty because of mismatch file hash %ls vs %ls", *MatchingDependency->FileHash, *RemoteDependency.FileHash);
			bIsDirty = true;
		}

		// File present locally but dirty, this is an overwrite
		if (bIsDirty)
		{
			FStormSyncFileModifierInfo Modifier;
			Modifier.ModifierOperation = EStormSyncModifierOperation::Overwrite;
			Modifier.FileDependency = *MatchingDependency;
			UE_LOGF(LogStormSyncCore, Verbose, "\tAdding Modifier: %ls", *Modifier.ToString());
			Modifiers.Add(Modifier);
		}
	}

	return Modifiers;
}

void FStormSyncCoreUtils::RecursiveGetDependencies(const FName& InPackageName, TArray<FName>& OutAllDependencies)
{
	TArray<FName> Dependencies;
	TArray<FAssetData> Assets;
	if (!GetAssetData(InPackageName.ToString(), Assets, Dependencies))
	{
		UE_LOGF(LogStormSyncCore, Warning, "FStormSyncCoreUtils::RecursiveGetDependencies - GetAssetData failed to load assets for %ls", *InPackageName.ToString());
		return;
	}

	for (const FName& Dependency : Dependencies)
	{
		const FString DependencyName = Dependency.ToString();
		UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::RecursiveGetDependencies - Gather dependencies for %ls", *DependencyName);

		if (IsValidDependency(DependencyName))
		{
			if (!OutAllDependencies.Contains(Dependency))
			{
				OutAllDependencies.Add(Dependency);
				RecursiveGetDependencies(Dependency, OutAllDependencies);
			}
		}
	}
}

bool FStormSyncCoreUtils::IsValidDependency(const FString& InDependencyName)
{
	UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::IsValidDependency - Check dependency (%ls)", *InDependencyName);

	const UStormSyncCoreSettings* Settings = GetDefault<UStormSyncCoreSettings>();
	check(Settings);

	// Filter out any references outside of /Game if user opted to only export /Game content
	if (Settings->bExportOnlyGameContent && !InDependencyName.StartsWith(TEXT("/Game")))
	{
		UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::IsValidDependency - Filter out dependency \"%ls\". Ignored by bExportOnlyGameContent setting", *InDependencyName);
		return false;
	}

	// Build the list of ignores, eg. 
	TArray<FName> IgnoredPackages = Settings->IgnoredPackages.Array();
	IgnoredPackages.Append(Settings->IgnoredPackagesInternal.Array());

	// Filter out any ignored packages
	for (const FName IgnoredPackageName : IgnoredPackages)
	{
		if (InDependencyName.StartsWith(IgnoredPackageName.ToString()))
		{
			UE_LOGF(LogStormSyncCore, Verbose, "FStormSyncCoreUtils::IsValidDependency - Filter out dependency \"%ls\". Ignored by \"%ls\" IgnoredPackages setting", *InDependencyName, *IgnoredPackageName.ToString());
			return false;
		}
	}

	return true;
}

bool FStormSyncCoreUtils::ValidateAssets(const TArray<FName>& InAssetsFilename, FText& OutErrorText)
{
	for (const FName& PackageName : InAssetsFilename)
	{
		FString PackageFilename;
		FString PackageNameStr = PackageName.ToString();
		if (!FPackageName::DoesPackageExist(PackageNameStr, &PackageFilename))
		{
			OutErrorText = FText::Format(LOCTEXT("ValidateAssets_Asset_Invalid", "{0} does not exist on disk."), FText::FromString(PackageNameStr));
			return false;
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
