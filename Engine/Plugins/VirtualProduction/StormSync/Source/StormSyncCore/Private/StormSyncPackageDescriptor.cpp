// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncPackageDescriptor.h"
#include "Internationalization/Internationalization.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "StormSyncPackageDescriptor"

FString FStormSyncFileDependency::GetDestFilepath(const FName& InPackageName, FText& OutFailureReason)
{
	const FString PackageNamePath = InPackageName.ToString();
	if (PackageNamePath.IsEmpty())
	{
		// Early return out in case PackageName hasn't been initialized properly
		OutFailureReason = LOCTEXT("PackageName_Empty", "Couldn't determine destination filepath, PackageName is empty.");
		return {};
	}

	FString DestFilepath;

	// Try to get absolute filename from the provided package name, this will effectively check
	// for all registered mount points and return the correct value if root path of the package name
	// is matching a mount point (either plugin, /Game, or an arbitrary mount point registered by
	// StormSyncDrives module.
	if (!FPackageName::TryConvertLongPackageNameToFilename(PackageNamePath, DestFilepath))
	{
		OutFailureReason = FText::Format(
			LOCTEXT("PackageName_Invalid", "Couldn't determine Mount point and Relative path for \"{0}\"."),
			FText::FromName(InPackageName)
		);
		return {};
	}

	// Convert the path to a full path and attempt to convert to a mounted path again to validate it's a valid mount point. (see UE-368657)
	FPackageName::EErrorCode ErrorCode;
	if (!FPackageName::TryConvertToMountedPath(FPaths::ConvertRelativePathToFull(DestFilepath), 
		/*OutLocalPathNoExtension*/nullptr, 
		/*OutPackageName*/nullptr, 
		/*OutObjectName*/nullptr, 
		/*OutSubObjectName*/nullptr, 
		/*OutExtension*/nullptr, 
		/*OutFlexNameType*/nullptr, 
		&ErrorCode))
	{
		OutFailureReason = FPackageName::FormatErrorAsText(PackageNamePath, ErrorCode);
		return {};
	}

	return DestFilepath;
}

bool FStormSyncFileDependency::IsValid() const
{
	return Timestamp != 0 && FileSize != 0 && !FileHash.IsEmpty();
}

FString FStormSyncFileDependency::ToString() const
{
	return FString::Printf(TEXT("PackageName: %s, Timestamp: %lld, FileSize: %lld, FileHash: %s"), *PackageName.ToString(), Timestamp, FileSize, *FileHash);
}

FString FStormSyncPackageDescriptor::ToString() const
{
	return FString::Printf(TEXT("Name: %s, Version: %s, Description: %s, Files: %d"), *Name, *Version, *Description, Dependencies.Num());
}

FString FStormSyncFileModifierInfo::ToString() const
{
	return FString::Printf(TEXT("ModifierOperation: %s, FileDependency: %s"), *UEnum::GetValueAsString(ModifierOperation), *FileDependency.ToString());
}

#undef LOCTEXT_NAMESPACE
