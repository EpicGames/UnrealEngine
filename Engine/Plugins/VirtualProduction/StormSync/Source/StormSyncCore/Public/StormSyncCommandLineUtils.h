// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"

class FName;

/** Provides command line helpers to help in dealing with command parameters */
class STORMSYNCCORE_API FStormSyncCommandLineUtils
{
public:

	/**
	 * Parses a string into tokens, separating switches (beginning with - or /) from
	 * other parameters
	 *
	 * @param	CmdLine		the string to parse
	 * @param	Arguments	[out] filled with all the non-option arguments found in the string
	 */
	static void Parse(const TCHAR* CmdLine, TArray<FName>& Arguments);
};
