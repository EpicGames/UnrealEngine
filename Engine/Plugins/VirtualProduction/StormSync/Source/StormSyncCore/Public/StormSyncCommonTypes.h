// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "StormSyncCommonTypes.generated.h"

class FArchive;

// Common alias to a ThreadSafe SharedPtr to hold a Buffer (array of uint8)
using FStormSyncBuffer = TArray64<uint8>;
using FStormSyncBufferPtr = TSharedPtr<FStormSyncBuffer, ESPMode::ThreadSafe>;
using FStormSyncArchivePtr = TSharedPtr<FArchive, ESPMode::ThreadSafe>;

/** Engine type values for a storm sync connected device */
UENUM()
enum class EStormSyncEngineType : uint8
{
	Server,
	Commandlet,
	Editor,
	Game,
	Other,
	Unknown
};
