// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StormSyncCommonTypes.h"
#include "StormSyncPackageDescriptor.h"
#include "StormSyncCoreDelegates.generated.h"

/** Connection states for storm sync discovered devices over the network */
UENUM()
enum class EStormSyncConnectedDeviceState : uint8
{
	/** Connected and known from this editor instance */
	State_Active,

	/** Became unresponsive, haven't received any heartbeats for a certain amount of time (as configured in project settings) */
	State_Unresponsive,

	/** Initial state before service discovery has a chance to update state */
	State_Disconnected
};

/** UI related data holder for storm sync connected devices */
struct FStormSyncConnectedDevice
{
	/** Holds the state of the connection*/
	EStormSyncConnectedDeviceState State = EStormSyncConnectedDeviceState::State_Disconnected;
	
	/**
	 * Holds the state of the remote server for the connection.
	 *
	 * Whether remote server endpoint is currently running and able to receive incoming connections
	 */
	bool bIsServerRunning = false;
	
	/** Holds the message bus address identifier for Service Discovery endpoint */
	FString MessageAddressId;
	
	/** Holds the message bus address identifier for Storm Sync Server endpoint */
	FString StormSyncServerAddressId;
	
	/** Holds the message bus address identifier for Storm Sync Client endpoint */
	FString StormSyncClientAddressId;
	
	/** The hostname this message was generated from */
	FString HostName;

	/** The unreal project name this message was generated from */
	FString ProjectName;
	
	/** The unreal project directory this message was generated from (last portion of the path) */
	FString ProjectDir;

	/** Holds instance identifier */
	FString InstanceId;

	/** Holds the type of the engine instance. */
	EStormSyncEngineType InstanceType = EStormSyncEngineType::Unknown;

	/** Default constructor */
	FStormSyncConnectedDevice() = default;
};

/**
 * FStormSyncCoreDelegates
 * 
 * Delegates used by the editor.
 */
struct STORMSYNCCORE_API FStormSyncCoreDelegates
{
	/**
	 * Delegate type for when an incoming pak extraction process starts
	 *
	 * @param PackageDescriptor Metadata information about incoming buffer (name, description, etc.)
	 * @param Archive The archive for the incoming pak buffer
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRequestImportBuffer, const FStormSyncPackageDescriptor&, const FStormSyncArchivePtr&);

	/**
	 * Delegate type for when one an incoming pak extraction process starts
	 *
	 * @param Filename The absolute filename path for the imported file
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRequestImportFile, const FString&);

	/**
	 * Delegate type for when imported spak has completed extraction.
	 *
	 * @param InFilename The absolute filename path for the imported file
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnFileImported, const FString& /*InFilename*/);

	/**
	 * Delegate type for when one asset is extracted from an incoming pak
	 *
	 * @param PackageName Original Package Name
	 * @param DestFilepath The fully qualified destination filepath where the file should be extracted to
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPakAssetExtracted, const FName& /*PackageName*/, const FString& /*DestFilepath*/);

	/**
	 * Delegate type for a client socket about to open a stream for sending a buffer
	 *
	 * @param RemoteAddress FString remote address of the server we are about to send to
	 * @param RemoteHostName FString remote hostname (or friendly name) of the server we are sending to
	 * @param PackageNums The total number of files (or package names) to include in the buffer
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPreStartSendingBuffer, const FString&, const FString&, int32);

	/**
	 * Delegate type for a client socket opening a tcp stream and sending a buffer to a remote server
	 *
	 * @param RemoteAddress FString remote address of the server we are sending to
	 * @param BufferSize The total number of bytes the server should expect, representing the buffer size
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnStartSendingBuffer, const FString&, int32);

	/**
	 * Delegate type for a client socket receiving response from server, indicating how many bytes it received so far
	 *
	 * @param RemoteAddress FString remote address of the server we get a response from
	 * @param BytesCount The number of bytes received so far by the remote server
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReceivingBytes, const FString&, int32);

	/**
	 * Delegate type for a client socket receiving "transfer complete" response from server, indicating the tcp transfer is done.
	 *
	 * @param RemoteAddress FString remote address of the server we get a response from
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTransferComplete, const FString&);

	/**
	 * Delegate type for a new connection over the network on storm sync message bus.
	 *
	 * Service Discovery related.
	 *
	 * @param FString Message Bus Address UID.
	 * @param FStormSyncConnectedDevice Data Holding struct with info about connected device.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnServiceDiscoveryConnection, const FString&, const FStormSyncConnectedDevice&);

	/**
	 * Delegate type for when a connection on storm sync network state is changing (might become unresponsive).
	 *
	 * Service Discovery related.
	 *
	 * @param FString Message Bus Address UID.
	 * @param EStormSyncConnectedDeviceState Enum state indicating if connection is active or unresponsive
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnServiceDiscoveryStateChange, const FString&, EStormSyncConnectedDeviceState);


	/**
	 * Wakeup event sent through the discovery service to re-active the other services.
	 */
	DECLARE_MULTICAST_DELEGATE(FOnServiceDiscoveryReceivedWakeup);

	/**
	 * Delegate type for when server status for a connection on storm sync network has changed (either running or stopped).
	 *
	 * Service Discovery related.
	 *
	 * @param FString Message Bus Address UID.
	 * @param boolean Whether the state for remote server endpoint is running or stopped (false)
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnServiceDiscoveryServerStatusChange, const FString&, bool);

	/** Broadcasted when server module starts the endpoint */
	DECLARE_MULTICAST_DELEGATE(FOnStormSyncServerStarted)

	/** Broadcasted when server module stops the endpoint */
	DECLARE_MULTICAST_DELEGATE(FOnStormSyncServerStopped)


	/**
	 * Delegate type for a disconnection on storm sync message bus.
	 *
	 * Happens after a certain amount of inactivity configured via StormSyncTransportSettings.
	 *
	 * Service Discovery related.
	 *
	 * @param FString Message Bus Address UID.
	 * @param FStormSyncConnectedDevice Data Holding struct with info about connected device.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnServiceDiscoveryDisconnection, const FString&);

	/** Called when receiving a buffer over network and used to queue up an import task */
	static FOnRequestImportBuffer OnRequestImportBuffer;

	/** Called when importing an .spak file via the Archive factory (import via content browser) and used to queue up an import task */
	static FOnRequestImportFile OnRequestImportFile;

	static FOnFileImported OnFileImported;

	/** Called when a file is extracted from an incoming storm sync pak */
	static FOnPakAssetExtracted OnPakAssetExtracted;

	/**
	 * Called when a tcp socket client is about to open a tcp stream and starts sending buffer to a remote server,
	 * before the buffer is created and the connection is established.
	 */
	static FOnPreStartSendingBuffer OnPreStartSendingBuffer;

	/** Called when a tcp socket client opens a tcp stream and starts sending a buffer to a remote server */
	static FOnStartSendingBuffer OnStartSendingBuffer;

	/** Called when a tcp socket client is receiving a response from a server we are sending to (including the size it receives so far) */
	static FOnReceivingBytes OnReceivingBytes;

	/** Called when a tcp socket client is receiving a response from a server we are sending to (including the size it receives so far) */
	static FOnTransferComplete OnTransferComplete;

	/** Called when service discovery manager detects a new connection on Storm Sync network */
	static FOnServiceDiscoveryConnection OnServiceDiscoveryConnection;

	/**
	 * Called when service discovery manager detects a state change for a connected device on
	 * Storm Sync network (might become unresponsive)
	 */
	static FOnServiceDiscoveryStateChange OnServiceDiscoveryStateChange;

	/**
	 * Called when the discovery service receives a wakeup request.
	 */
	static FOnServiceDiscoveryReceivedWakeup OnServiceDiscoveryReceivedWakeup;

	/** Called when service discovery manager detects a state change for remote server endpoint on a connected device */
	static FOnServiceDiscoveryServerStatusChange OnServiceDiscoveryServerStatusChange;

	/**
	 * Called when service discovery manager detects a connection dropped on Storm Sync network
	 * (happens after a certain amount of inactivity configured via StormSyncTransportSettings)
	 */
	static FOnServiceDiscoveryDisconnection OnServiceDiscoveryDisconnection;

	/* Server started delegate */
	static FOnStormSyncServerStarted OnStormSyncServerStarted;

	/* Server stopped delegate */
	static FOnStormSyncServerStopped OnStormSyncServerStopped;
};
