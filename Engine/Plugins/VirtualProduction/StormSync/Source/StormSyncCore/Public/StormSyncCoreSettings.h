// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Engine/DeveloperSettings.h"
#include "StormSyncCoreSettings.generated.h"

/**
 * Settings for the StormSyncCore plugin modules.
 *
 * Handle configuration for buffer creation / extraction.
 */
UCLASS(MinimalAPI, Config=Game, DefaultConfig, DisplayName="Core Settings")
class UStormSyncCoreSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UStormSyncCoreSettings();

	/**
	 * Whether StormSync should consider packages only within the `/Game` project content when performing
	 * an export or sending over the network.
	 *
	 * Can be used together with IgnoredPackages list below.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings | Export")
	bool bExportOnlyGameContent = true;

	/**
	 * Whether to always ignore invalid references (eg. not existing on disk) during pak creation (when exporting or sending over network)
	 *
	 * If set to false, the export process will be stricter and prevent the operation, but you'll get a notification error about
	 * each invalid references.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings | Export", DisplayName = "Filter Out Invalid References")
	bool bFilterInvalidReferences = true;

	/**
	 * Set of package names to ignore in pak file creation.
	 *
	 * Any package dependencies is filtered out using this list. This is a StartsWith search pattern,
	 * meaning any value here should be the beginning of dependency package names we'd like to ignore
	 * (eg, /SomePlugin, /AnotherPlugin, /Game/Folder, ...)
	 *
	 * Can be used together with bExportOnlyGameContent set to true, in which case you can filter out specific
	 * /Game/... folders and paths (other root paths such as /SomePlugin would have no effect)
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings | Export")
	TSet<FName> IgnoredPackages;

	/**
	 * A DateTime format string to customize the default name generated during a package export to local file (.spak).
	 *
	 * If the format string is set empty, no date time information will be appended to the default export name.
	 *
	 * Uses a non-standard format syntax (see below)
	 *		%a - am or pm
	 *		%A - AM or PM
	 *		%d - Day, 01-31
	 *		%D - Day of the Year, 001-366
	 *		%m - Month, 01-12
	 *		%y - Year, YY
	 *		%Y - Year, YYYY
	 *		%h - 12h Hour, 01-12
	 *		%H - 24h Hour, 00-23
	 *		%M - Minute, 00-59
	 *		%S - Second, 00-60
	 *		%s - Millisecond, 000-999
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings | Export")
	FString ExportDefaultNameFormatString;

	/**
	 * Not exposed to end user via Project Settings UI, acts as a fail-safe additional ignore list for
	 * Packages like `/Engine` and `/Script`, or any other Packages we want to make sure to ignore regardless of user settings.
	 */
	TSet<FName> IgnoredPackagesInternal;

	/**
	 * Enables Hot Reloading of packages when modified by a sync operation.
	 *
	 * Hot reloading attempts to quickly reload the modified packages to avoid temporary
	 * invalid states to propagate in the system. If this is disabled, the assets
	 * will be deleted from memory and packages will be reloaded eventually
	 * by the Asset Registry's directory watcher.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Settings | Import")
	bool bEnableHotReloadPackages = true;
};
