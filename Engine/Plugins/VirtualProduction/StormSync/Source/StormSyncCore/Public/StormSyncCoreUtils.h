// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Delegates/Delegate.h"
#include "StormSyncCommonTypes.h"
#include "StormSyncPackageDescriptor.h"

/** 
 * FStormSyncCoreExtractArgs
 * 
 * Delegates used by the main extract method. Use it to register delegates when you want to be notified
 * from external code.
 */
struct FStormSyncCoreExtractArgs
{
	/**
	 * Delegate type for when one an incoming pak extraction process starts
	 * @param FileCount The int32 file count extracted from incoming buffer
	 */
	DECLARE_DELEGATE_OneParam(FOnPakPreExtract, int32 /*InFileCount*/);

	/**
	 * Delegate type for when one an incoming pak extraction is ended
	 * @param FileCount The int32 file count extracted from incoming buffer
	 */
	DECLARE_DELEGATE_OneParam(FOnPakPostExtract, int32 /*InFileCount*/);

	/**
	 * Delegate type called to obtain an archive to extract an asset. This is called before extraction starts.
	 * @param InFileDependency struct holding info such as Package Name, File size, timestamp and hash
	 * @param InDestFilename The fully qualified destination filepath where the file should be extracted to
	 * @return Archive (writer) into which the incoming file will be written.
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(FStormSyncArchivePtr, FOnGetArchiveForExtract, const FStormSyncFileDependency& /*InFileDependency*/, const FString& /*InDestFilename*/);

	/**
	 * Delegate type called after a file buffer is extracted from an incoming pak into the provided archive.
 	 * @param InFileDependency struct holding info such as Package Name, File size, timestamp and hash
	 * @param InDestFilename The fully qualified destination filepath where the file should be extracted to
	 * @param InArchive The archive that was returned by FOnGetArchiveForExtract is returned back.
	 */
	DECLARE_DELEGATE_ThreeParams(FOnArchiveExtracted, const FStormSyncFileDependency& /*InFileDependency*/, const FString& /*InDestFilename*/, const FStormSyncArchivePtr& /*InArchive*/);
	
	/**
	 * Delegate type for when one asset is being extracted from an incoming pak
	 * @param FileDependency struct holding info such as Package Name, File size, timestamp and hash
	 * @param InDestFilename The fully qualified destination filepath where the file should be extracted to
	 * @param InBuffer The raw buffer (as an array of bytes shared ptr) for the individual files extracted from incoming pak buffer
	 * @remark if an archive was provided by OnGetArchiveForExtract delegate, this delegate will not be called.
	*/
	DECLARE_DELEGATE_ThreeParams(FOnFileExtract, const FStormSyncFileDependency& /*FileDependency*/, FString /*InDestFilename*/, const FStormSyncBufferPtr& /*InBuffer*/);
	
	/** Delegate type for when one an incoming pak extraction process starts */
	FOnPakPreExtract OnPakPreExtract;

	/** Delegate type for when one an incoming pak extraction is ended */
	FOnPakPostExtract OnPakPostExtract;

	/** Delegate type called to obtain an archive to extract an asset. This is called before extraction starts. */
	FOnGetArchiveForExtract OnGetArchiveForExtract;

	/** Delegate type called after a file buffer is extracted from the archive. */
	FOnArchiveExtracted OnArchiveExtracted;

	/** Delegate type triggered when a file asset is extracted from an incoming pak */
	FOnFileExtract OnFileExtract;

	/** Default constructor */
	FStormSyncCoreExtractArgs() = default;
};

/** This class exposes a set of static methods to manipulate Ava Pak files, creation and extraction. */
class FStormSyncCoreUtils
{
public:
	/** Delegate type triggered on pak buffer creation for each added file */
	DECLARE_DELEGATE_OneParam(FOnFileAdded, const FStormSyncFileDependency&)

	/**
	 * Helper function to retrieve the FAssetData associated with a given file.
	 *
	 * Using AssetRegistry LoadPackageRegistryData() instead of GetDependencies() to ensure a consistent behavior between editor and -game mode.
	 *
	 * @param InPackageName PackageName (eg. /Game/Path/Asset) to base the operation on
	 * @param OutAssets The associated FAssetData (FLoadPackageRegistryData.Data) assets from LoadPackageRegistryData() call
	 * @param OutDependencies List of package dependencies
	 * 
	 * @return Whether the asset data could be retrieved or not
	 */
	STORMSYNCCORE_API static bool GetAssetData(const FString& InPackageName, TArray<FAssetData>& OutAssets, TArray<FName>& OutDependencies);

	/**
	 * Gathers a list of PackageNames dependencies referenced by the supplied packages.
	 *
	 * Internally uses the AssetRegistry to recursively determine dependencies and returns a flatten list of all
	 * referenced assets (On disk references ONLY).
	 *
	 * @param InPackageNames List of packages to look for dependencies
	 * @param OutDependencies Sorted (alphabetically) list of all package dependencies
	 * @param OutErrorText Localized Text if the operation failed
	 * @param bInShouldValidatePackages Whether to disable strict verification on validity of package (eg. must exist locally)
	 *
	 * @return Returns true if we were able to generate a list of dependencies and false if the operation failed.
	 * Note that in case of failure, ErrorText is filled with further information.
	 */
	STORMSYNCCORE_API static bool GetDependenciesForPackages(const TArray<FName>& InPackageNames, TArray<FName>& OutDependencies, FText& OutErrorText, const bool bInShouldValidatePackages = true);

	/**
	 * Similar to GetDependenciesForPackages internally searching for any recursive dependencies.
	 *
	 * Gathers a list of PackageNames dependencies referenced by the supplied packages,
	 * creating a list of FStormSyncFileDependency with info such as asset path, file size, timestamp and file hash.
	 *
	 * @param InPackageNames List of packages to look for dependencies
	 * @param OutFileDependencies Sorted (alphabetically) list of all package dependencies
	 * @param OutErrorText Localized Text if the operation failed
	 * @param bInShouldValidatePackages Whether to disable strict verification on validity of package (eg. must exist locally)
	 *
	 * @return Returns true if we were able to generate a list of dependencies and false if the operation failed.
	 * Note that in case of failure, ErrorText is filled with further information.
	 */
	STORMSYNCCORE_API static bool GetAvaFileDependenciesForPackages(const TArray<FName>& InPackageNames, TArray<FStormSyncFileDependency>& OutFileDependencies, FText& OutErrorText, const bool bInShouldValidatePackages = true);

	/**
	 * Creates a list of FStormSyncFileDependency (with info such as asset path, file size, timestamp and file hash)
	 * from the supplied list of package names.
	 *
	 * @param InPackageNames List of packages to look for dependencies
	 *
	 * @return Returns a list of FStormSyncFileDependency
	 */
	STORMSYNCCORE_API static TArray<FStormSyncFileDependency> GetAvaFileDependenciesFromPackageNames(const TArray<FName>& InPackageNames, bool bInShouldIncludeInvalid = false);

	/**
	 * Async version of FStormSyncCoreUtils::GetAvaFileDependenciesFromPackageNames to return a list of
	 * File Dependencies (with info about file size, asset path, timestamp file hash, etc.) from the provided
	 * list of package names.
	 * 
	 * @param InPackageNames The list of package names to convert into FStormSyncFileDependency
	 * @param bInShouldValidatePackages Whether to disable strict verification on validity of package (eg. must exist locally)
	 * @param InThreadType Optional Thread Type to use for the async execution (Default EAsyncExecution::Thread)
	 * @return A future containing the list of file dependencies.
	 */
	STORMSYNCCORE_API static TFuture<TArray<FStormSyncFileDependency>> GetAvaFileDependenciesAsync(const TArray<FName>& InPackageNames, const bool bInShouldValidatePackages = true, EAsyncExecution InThreadType = EAsyncExecution::Thread);

	/**
	 * Creates a new FStormSyncFileDependency and initialize filesystem related information
	 *
	 * Note: Considering renaming FStormSyncFileDependency to FStormSyncFileInfo (to distinguish with file references / dependencies idea)
	 */
	STORMSYNCCORE_API static FStormSyncFileDependency CreateStormSyncFile(const FName& InPackageName);

	/**
	 * Recursively get dependencies for provided list of Package Names and attempts to create an ava pak buffer using a
	 * memory archive.
	 *
	 * A list of FStormSyncFileDependency is also returned with `OutSuccessfullyPackedFiles` including info such as asset path,
	 * file size, timestamp and file hash.
	 *
	 * @param InPackageNames List of package name to include in the pak
	 * @param OutPakBuffer Raw buffer of the created in memory archive
	 * @param OutErrorText Localized Text if the operation failed
	 * @param InOnFileAdded Delegate triggered with FStormSyncFileDependency for each added file
	 *
	 * @see FStormSyncFileDependency
	 *
	 * @return Returns true if we were able to generate the pak buffer false if the operation failed.
	 * Note that in case of failure, ErrorText is filled with further information.
	 */
	STORMSYNCCORE_API static bool CreatePakBufferWithDependencies(const TArray<FName>& InPackageNames, FArchive& OutPakArchive, FText& OutErrorText, const FOnFileAdded& InOnFileAdded = FOnFileAdded());

	/**
	 * Attempts to create an ava pak buffer using a memory archive from the provided list of Package Names.
	 *
	 * A list of FStormSyncFileDependency is also returned with `OutSuccessfullyPackedFiles` including info such as asset path,
	 * file size, timestamp and file hash.
	 *
	 * It differs from CreatePakBufferWithDependencies by not looking for inner dependencies. If you need to recursively search for dependencies,
	 * use CreatePakBufferWithDependencies instead.
	 *
	 * @param InPackageNames List of package name to include in the pak
	 * @param OutPakArchive Destination archive
	 * @param OutErrorText Localized Text if the operation failed
	 * @param InOnFileAdded Delegate triggered with FStormSyncFileDependency for each added file
	 * 
	 * @see FStormSyncFileDependency
	 *
	 * @return Returns true if we were able to generate the pak buffer false if the operation failed.
	 * Note that in case of failure, ErrorText is filled with further information.
	 */
	STORMSYNCCORE_API static bool CreatePakBuffer(const TArray<FName>& InPackageNames, FArchive& OutPakArchive, FText& OutErrorText, const FOnFileAdded& InOnFileAdded = FOnFileAdded());

	/**
	 * Main entry point for unpacking. Extracts an ava pak into the current Unreal project (Game content or Plugins content folder)
	 * 
	 * Will extract package names from pak buffer, and create any .uasset files that does not exist yet on local project.
	 *
	 * @param InPakArchive Raw ava pak archive to extract into project
	 * @param InExtractArgs The arguments used to trigger delegates during extraction process
	 * @param OutErrors List of localized Text if the operation failed
	 *
	 * @return Returns true if we were able to fully import the package and false if the operation failed (even for partial imports).
	 * Note that in case of failure, Errors array of localized text is filled with further information.
	 */
	STORMSYNCCORE_API static bool ExtractPakBuffer(FArchive& InPakArchive, const FStormSyncCoreExtractArgs& InExtractArgs, TArray<FText>& OutErrors);

	/**
	 * Converts the provided bytes size into a human readable string.
	 *
	 * @param InSize Number of bytes to convert
	 *
	 * @return A string representing the provided size with B, KB, MB of GB suffix.
	 */
	STORMSYNCCORE_API static FString GetHumanReadableByteSize(uint64 InSize);

	/**
	 * Figures out the files that needs to be synced based on local and remote list of dependencies.
	 *
	 * Returns a list of FStormSyncFileModifierInfo based on the provided list of file dependencies (most likely
	 * coming from remote and part of sync request / response message transaction) from which we extract package
	 * names and gather the local state via GetAvaFileDependenciesFromPackageNames().
	 *
	 * A list of FStormSyncFileModifierInfo is then based on those two list of file dependencies:
	 *
	 * - If missing locally, a FStormSyncFileModifierInfo is added with Addition operation
	 * - If present but with mismatched FileSize, a FStormSyncFileModifierInfo is added with Overwrite operation
	 * - If present but with mismatched FileHash, a FStormSyncFileModifierInfo is added with Overwrite operation
	 * - If present but with exact same FileSize and FileHash, no FStormSyncFileModifierInfo is added
	 *
	 * @param InPackageNames List of top level package names (from which we gather references) to base the diffing on
	 * @param InRemoteDependencies List of FStormSyncFileDependency to base our diffing against
	 *
	 * @return List of FStormSyncFileModifierInfo indicating files that are either missing or dirty
	 */
	STORMSYNCCORE_API static TArray<FStormSyncFileModifierInfo> GetSyncFileModifiers(const TArray<FName>& InPackageNames, const TArray<FStormSyncFileDependency>& InRemoteDependencies);

private:
	/**
	 * Gets the dependencies of the specified package recursively.
	 *
	 * Note: Closely follow RecursiveGetDependencies implementation of AssetTools, except we don't deal with External Actors here.
	 *
	 * Uses AssetRegistry to gather up all dependency for a given package name.
	 */
	static void RecursiveGetDependencies(const FName& InPackageName, TArray<FName>& OutAllDependencies);

	/** Filters out any dependency we don't want to consider for a pak */
	static bool IsValidDependency(const FString& InDependencyName);

	/** Validates all assets for existence on disk */
	static bool ValidateAssets(const TArray<FName>& InAssetsFilename, FText& OutErrorText);
};
