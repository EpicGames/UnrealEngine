// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "StormSyncPackageDescriptor.h"
#include "StormSyncImportTypes.generated.h"

/**
 * Data holder during an extraction process.
 *
 * Used to compute a list of files that needs extraction.
 */
USTRUCT()
struct FStormSyncImportFileInfo
{
	GENERATED_BODY()
	
	/** Holds info such as PackageName, FileSize, Timestamp and hash*/
	FStormSyncFileDependency FileDependency;

	/** The associated fully qualified path of the .uasset */
	FString DestFilepath;

	/** UI-exposed reason of the import */
	FText ImportReason;
	
	/** UI-exposed extended info for reason of the import */
	FText ImportReasonTooltip;

	/** True if the file was a new asset */
	bool bNewAsset = false;

	FStormSyncImportFileInfo() = default;

	FStormSyncImportFileInfo(const FStormSyncFileDependency& InFileDependency, const FString& InDestFilepath)
		: FileDependency(InFileDependency)
		, DestFilepath(InDestFilepath)
	{
	}
};
