// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StormSyncPackageDescriptor.generated.h"

/**
 * FStormSyncFileDependency.
 *
 * Represents a file included in an ava package (and buffer). Has information about package name, and file system data
 * such as file timestamp, size and hash used in the diffing process to figure out which files need to be synchronized
 * for a given ava package.
 */
USTRUCT()
struct FStormSyncFileDependency
{
	GENERATED_BODY()

	/** Default constructor. */
	FStormSyncFileDependency() = default;

	/** Default constructor initializing just the PackageName */
	explicit FStormSyncFileDependency(const FName& InPackageName)
		: PackageName(InPackageName)
	{
	}

	/** Default constructor with all members initialization */
	FStormSyncFileDependency(const FName& InPackageName, const int32 InFileSize, const int64 InTimestamp, const FString& InFileHash)
		: PackageName(InPackageName)
		, FileSize(InFileSize)
		, Timestamp(InTimestamp)
		, FileHash(InFileHash)
	{
	}

	/** Original package name (eg. /Game/Path/File) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	FName PackageName;

	/** File size on disk */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	uint64 FileSize = 0;

	/** Unix timestamp the file had when the buffer was created */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	int64 Timestamp = 0;

	/** The MD5 hash of the file on disk */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	FString FileHash;

	/**
	 * Return the fully qualified path for destination filename based on provided Package Name.
	 * The path on disk where this asset should be extracted to. If a package is coming from a
	 * plugin, it'll return the correct content destination folder for the plugin if it is
	 * installed in current project.
	 *
	 * It also handles arbitrary mount points and returns the absolute filename based on registered mount points.
	 *
	 * @param InPackageName Package name to base
	 * @param OutFailureReason Localized text filled with more information if operation failed.
	 *
	 * @return Destination filepath or empty string if we were not able to determine destination output
	 * (eg, Plugin package while plugin not installed or enabled in current project)
	 */
	STORMSYNCCORE_API static FString GetDestFilepath(const FName& InPackageName, FText& OutFailureReason);

	/** Returns whether file info was properly initialized */
	STORMSYNCCORE_API bool IsValid() const;

	STORMSYNCCORE_API FString ToString() const;
};

/**
 * FStormSyncPackageDescriptor.
 *
 * A package descriptor is holding the metadata information for a given ava package. Has information about its name,
 * description, version, author and most importantly a list of file dependencies (full list of files with data about
 * their state, timestamp, size, hash, etc.)
 */
USTRUCT()
struct FStormSyncPackageDescriptor
{
	GENERATED_BODY()

	/** Default constructor. */
	FStormSyncPackageDescriptor()
		: Author(FPlatformProcess::UserName())
	{
	}

	/** Friendly name of the package */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	FString Name;

	/**
	 * Name of the version for this plugin.
	 *
	 * This is the front-facing part of the version number. It doesn't need to match the version number numerically, but should
	 * be updated when the version number is increased accordingly.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	FString Version = TEXT("1.0.0");

	/** Description of the package */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	FString Description;

	/** The company or individual who created this package */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	FString Author;

	/** The inner list of file assets dependencies for this ava package */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync")
	TArray<FStormSyncFileDependency> Dependencies;

	STORMSYNCCORE_API FString ToString() const;
};

/**
 * EStormSyncModifierOperation.
 *
 * Defines the ways that mods will modify packages
 *
 * @see FStormSyncFileModifierInfo
 */
UENUM()
enum class EStormSyncModifierOperation : uint8
{
	/** This is an addition, meaning receiver is missing the file */
	Addition UMETA(DisplayName="Add"),
	
	/** This is a missing situation, meaning receiver has the file but sender has not */
	Missing UMETA(DisplayName="Missing"),
	
	/** This is an overwrite, meaning both sender and receiver have the file, but in a different state (mismatch filesize, hash, etc.) */
	Overwrite UMETA(DisplayName="Overwrite"),
};

/**
 * FStormSyncFileModifierInfo.
 *
 * Effectively a sync request with an additional list of sync modifiers, representing the diffing results
 * and files we want to synchronize.
 */
USTRUCT()
struct FStormSyncFileModifierInfo
{
	GENERATED_BODY()

	/** The operation of this modifier: Addition, Missing, Overwrite, etc  */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	EStormSyncModifierOperation ModifierOperation = EStormSyncModifierOperation::Addition;

	/* The file dependency this modifier relates to */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FStormSyncFileDependency FileDependency;

	/** Default constructor. */
	FStormSyncFileModifierInfo() = default;

	FStormSyncFileModifierInfo(const EStormSyncModifierOperation InModifierOperation, const FStormSyncFileDependency& InFileDependency)
		: ModifierOperation(InModifierOperation)
		, FileDependency(InFileDependency)
	{
	}

	STORMSYNCCORE_API FString ToString() const;
};
