// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StormSyncCore : ModuleRules
{
	public StormSyncCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"Projects",
				"Serialization"
			}
		);
	}
}
