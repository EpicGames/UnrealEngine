// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncDrivesModule.h"

#include "Engine/DeveloperSettings.h"
#include "Misc/CoreDelegates.h"
#include "Misc/PackageName.h"
#include "StormSyncDrivesLog.h"
#include "StormSyncDrivesSettings.h"
#include "StormSyncDrivesUtils.h"

#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Algo/AnyOf.h"

#if WITH_EDITOR
#include "MessageLogModule.h"
#include "Logging/MessageLog.h"
#endif

#define LOCTEXT_NAMESPACE "FStormSyncDrivesModule"

void FStormSyncDrivesModule::StartupModule()
{
#if WITH_EDITOR
	// Create a message log for the validation to use
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowPages = true;
	MessageLogModule.RegisterLogListing(LogName, LOCTEXT("StormSyncLogLabel", "Storm Sync Drives"), InitOptions);
	LogListing = MessageLogModule.GetLogListing(LogName);
#endif

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStormSyncDrivesModule::OnPostEngineInit);

#if WITH_EDITOR
	GetMutableDefault<UStormSyncDrivesSettings>()->OnSettingChanged().AddRaw(this, &FStormSyncDrivesModule::OnSettingsChanged);
#endif
}

void FStormSyncDrivesModule::ShutdownModule()
{
#if WITH_EDITOR
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		// unregister message log
		FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing(LogName);
	}
#endif

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	UnregisterMountedDrives();
	
#if WITH_EDITOR
	if (UObjectInitialized())
	{
		GetMutableDefault<UStormSyncDrivesSettings>()->OnSettingChanged().RemoveAll(this);
	}
#endif

	check(CachedMountPoints.IsEmpty());
}

bool FStormSyncDrivesModule::RegisterMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode)
{
	UE_LOGF(LogStormSyncDrives, Display, "FStormSyncDrivesModule::RegisterMountPoint ... MountPoint: %ls, Path: %ls", *InMountPoint.MountPoint, *InMountPoint.MountDirectory.Path);

	// Validate first configuration of the mount point
	FText ValidationError;
	if (!FStormSyncDrivesUtils::ValidateMountPoint(InMountPoint, ValidationError, OutErrorCode))
	{
		ErrorText = FText::Format(
			LOCTEXT("ValidationError_MountPoint", "Validation failed for MountPoint (MountPoint: {0}, MountDirectory: {1}) - {2}"),
			FText::FromString(InMountPoint.MountPoint),
			FText::FromString(InMountPoint.MountDirectory.Path),
			ValidationError
		);
		AddMessageError(ErrorText);
		return false;
	}

	// Append a trailing slash so that it is consistent with FLongPackagePathsSingleton::ContentPathToRoot entries and avoid a crash
	// with assets and reimport path when right clicking on them

	// ValidateMountPoint should ensure no trailing slash is allowed in user config
	const FString MountPoint = InMountPoint.MountPoint + TEXT("/");

	// Check if root path is already mounted
	if (FPackageName::MountPointExists(MountPoint))
	{
		ErrorText = FText::Format(
			LOCTEXT("ValidationError_MountPointExists", "Mount Point {0} already exists"),
			FText::FromString(MountPoint)
		);
		AddMessageError(ErrorText);
		return false;
	}

	UE_LOGF(LogStormSyncDrives, Display, "FStormSyncDrivesModule::RegisterMountPoint ... Try register %ls to %ls", *MountPoint, *InMountPoint.MountDirectory.Path);
	FPackageName::RegisterMountPoint(MountPoint, InMountPoint.MountDirectory.Path);

	CachedMountPoints.FindOrAdd(InMountPoint.MountPoint) = InMountPoint;
	return true;
}

bool FStormSyncDrivesModule::UnregisterMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText)
{
	const FString ContentPath = InMountPoint.MountDirectory.Path;

	// Mount points are always added with a trailing slash (eg. /Foo => /Foo/)
	const bool bHasLeadingSlash = InMountPoint.MountPoint.EndsWith(TEXT("/"));
	const FString RootPath = bHasLeadingSlash ? InMountPoint.MountPoint : InMountPoint.MountPoint + TEXT("/");

	if (!FPackageName::MountPointExists(RootPath))
	{
		ErrorText = FText::Format(
			LOCTEXT("ValidationError_MountPointUnexisting", "Mount Point {0} does not exist"),
			FText::FromString(RootPath)
		);
		return false;
	}

	FPackageName::UnRegisterMountPoint(RootPath, ContentPath);

	CachedMountPoints.Remove(InMountPoint.MountPoint);
	return true;
}

TArray<FStormSyncMountPointConfig> FStormSyncDrivesModule::GetCommandLineConfigs() const
{
	TArray<FStormSyncMountPointConfig> Configs;
	FString AdditionalMountPointsString;
	if (FParse::Value(FCommandLine::Get(), TEXT("-STORMSYNCMOUNTPOINTS"), AdditionalMountPointsString))
	{
		// look over a list of cvars
		TArray<FString> AdditionalMountPoints;
		AdditionalMountPointsString.ParseIntoArray(AdditionalMountPoints, TEXT(","), true);
		for (const FString& AdditionalMountPoint : AdditionalMountPoints)
		{
			// split up each Key=Value pair
			FString AssetPath, Filepath;
			if (AdditionalMountPoint.Split(TEXT("="), &AssetPath, &Filepath))
			{
				if (!AssetPath.IsEmpty() && !Filepath.IsEmpty())
				{
					if (FPaths::IsRelative(Filepath))
					{
						Filepath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir(), Filepath);
					}

					FStormSyncMountPointConfig Config;
					Config.MountPoint = AssetPath;
					Config.MountDirectory.Path = Filepath;
					Configs.Add(MoveTemp(Config));
				}
			}
		}
	}
	
	return Configs;
}

void FStormSyncDrivesModule::OnPostEngineInit()
{
	UE_LOGF(LogStormSyncDrives, Display, "FStormSyncDrivesModule::OnPostEngineInit ...");
	UE_LOGF(LogStormSyncDrives, Display, "\t Mounting Drives based on config");

	ResetMountedDrivesFromSettings(GetDefault<UStormSyncDrivesSettings>());

}

namespace UE::StormSyncDrives::Private
{
TArray<FStormSyncMountPointConfig> RemoveDuplicatesAndConvertToAbsPaths(const TArray<FStormSyncMountPointConfig>& InMountPoints)
{
	TArray<FString> RootPaths;
	TArray<FString> MountDirectories;

	TArray<FStormSyncMountPointConfig> OutConfig;
	for (FStormSyncMountPointConfig Entry : InMountPoints)
	{
		// Test for Mount Points duplicates
		if (RootPaths.Contains(Entry.MountPoint) || MountDirectories.Contains(Entry.MountDirectory.Path))
		{
			continue;
		}

		if (FPaths::IsRelative(Entry.MountDirectory.Path))
		{
			Entry.MountDirectory.Path = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir(), Entry.MountDirectory.Path);
		}
		
		RootPaths.AddUnique(Entry.MountPoint);
		MountDirectories.AddUnique(Entry.MountDirectory.Path);

		OutConfig.Emplace(MoveTemp(Entry));
	}
	
	return OutConfig;
}

void ConformSettingsToUserPreference(UStormSyncDrivesSettings* InSettings)
{
	bool bNeedsSave = false;
	for (FStormSyncMountPointConfig& Entry : InSettings->MountPoints)
	{
		if (Entry.bResolveAsRelativePath && !FPaths::IsRelative(Entry.MountDirectory.Path))
		{
			FPaths::MakePathRelativeTo(Entry.MountDirectory.Path, *FPaths::ProjectContentDir());
			bNeedsSave = true;
		}
		else if (!Entry.bResolveAsRelativePath && FPaths::IsRelative(Entry.MountDirectory.Path))
		{
			Entry.MountDirectory.Path = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir(), Entry.MountDirectory.Path);
			bNeedsSave = true;
		}

		if (Entry.MountPoint.EndsWith(TEXT("/")))
		{
			Entry.MountPoint = Entry.MountPoint.LeftChop(1);
		}
	}

	if (bNeedsSave)
	{
		InSettings->SaveConfig();
	}
}

} // namespace UE::StormSyncDrives::Private


void FStormSyncDrivesModule::OnSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName PropertyName = InPropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = (InPropertyChangedEvent.MemberProperty != nullptr) ? InPropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const FName MountPointsPropertyName = GET_MEMBER_NAME_CHECKED(UStormSyncDrivesSettings, MountPoints);


	if (PropertyName == MountPointsPropertyName || MemberPropertyName == MountPointsPropertyName)
	{
		UStormSyncDrivesSettings* Settings = CastChecked<UStormSyncDrivesSettings>(InSettings);
		
		UE::StormSyncDrives::Private::ConformSettingsToUserPreference(Settings);
		ResetMountedDrivesFromSettings(Settings);
	}
}

bool FStormSyncDrivesModule::IsMountPointInCache(const FStormSyncMountPointConfig& InMountPoint) const
{
	return CachedMountPoints.Contains(InMountPoint.MountPoint) &&
		CachedMountPoints.FindChecked(InMountPoint.MountPoint).MountDirectory.Path == InMountPoint.MountDirectory.Path;
}

bool FStormSyncDrivesModule::MountDriveHelper(const FStormSyncMountPointConfig& InMountPoint)
{
	if (CachedMountPoints.Contains(InMountPoint.MountPoint))
	{
		FText ErrorText;
		if (!UnregisterMountPoint(InMountPoint, ErrorText))
		{
			UE_LOGF(LogStormSyncDrives, Error, "FStormSyncDrivesModule::UnregisterMountPoint failed with Error: %ls", *ErrorText.ToString());
			return false;
		}
	}

	FText ErrorText;
	if (!RegisterMountPoint(InMountPoint, ErrorText))
	{
		UE_LOGF(LogStormSyncDrives, Error, "FStormSyncDrivesModule::RegisterMountPoint failed with Error: %ls", *ErrorText.ToString());
		return false;
	}

	return true;
}

bool FStormSyncDrivesModule::RemoveFromCacheIfNotInConfigs(const TArray<FStormSyncMountPointConfig>& Configs)
{
	TArray<FStormSyncMountPointConfig> UnregisterList;
	
	// Collect all configs that should be unmounted. 
	for (const TPair<FString, FStormSyncMountPointConfig>& Cache : CachedMountPoints)
	{
		bool bInSettings = Algo::AnyOf(Configs, [&Key = Cache.Key](const FStormSyncMountPointConfig& InMount){ return InMount.MountPoint == Key; });
		if (!bInSettings)
		{
			UnregisterList.Add(Cache.Value);
		}
	}

	// Now try to unmount.
	bool bSuccess = true;
	for (const FStormSyncMountPointConfig& Config : UnregisterList)
	{
		FText ErrorText;
		if (!UnregisterMountPoint(Config, ErrorText))
		{
			UE_LOGF(LogStormSyncDrives, Error, "FStormSyncDrivesModule::RemoveFromCacheIfNotInConfigs failed with Error: %ls", *ErrorText.ToString());
			bSuccess = false;
		}
	}

	return bSuccess;
}

void FStormSyncDrivesModule::ResetMountedDrivesFromSettings(const UStormSyncDrivesSettings* InSettings)
{
	UE_LOGF(LogStormSyncDrives, Display, "FStormSyncDrivesModule::ResetMountedDrives ...");
	check(InSettings);

	// Collect any command line configs the user has specified. 
	TArray<FStormSyncMountPointConfig> Configs = GetCommandLineConfigs();
	Configs += InSettings->MountPoints;

	TArray<FStormSyncMountPointConfig> Filtered = UE::StormSyncDrives::Private::RemoveDuplicatesAndConvertToAbsPaths(Configs);
	UE_CLOGF(Filtered.Num() != Configs.Num(), LogStormSyncDrives, Warning, "FStormSyncDrivesModule::ResetMountedDrives settings object contains duplicate values.");

#if WITH_EDITOR
	LogListing->ClearMessages();
#endif

	// Mount any config not in our current cache. 
	bool bSuccess = true;
	for (const FStormSyncMountPointConfig& MountPoint : Filtered)
	{
		if (!IsMountPointInCache(MountPoint))
		{
			bSuccess &= MountDriveHelper(MountPoint);
		}
	}

	// Remove any stale cache items. 
	bSuccess &= RemoveFromCacheIfNotInConfigs(Filtered);

#if WITH_EDITOR
	if (!bSuccess)
 	{
		LogListing->NotifyIfAnyMessages(LOCTEXT("Validation_Error", "Storm Sync: Error validating Mount Points settings"), EMessageSeverity::Error, true);
 	}
#endif
}

void FStormSyncDrivesModule::UnregisterMountedDrives()
{
	UE_LOGF(LogStormSyncDrives, Display, "FStormSyncDrivesModule::UnregisterMountedDrives ...");
	for (const TPair<FString, FStormSyncMountPointConfig>& Cache : CachedMountPoints)
	{
		FText ErrorText;
		if (!UnregisterMountPoint(Cache.Value, ErrorText))
		{
			UE_LOGF(LogStormSyncDrives, Error, "FStormSyncDrivesModule::UnregisterMountedDrives failed with Error: %ls", *ErrorText.ToString());
		}
	}
}

void FStormSyncDrivesModule::AddMessageError(const FText& Text)
{
#if WITH_EDITOR
	FMessageLog MessageLog(LogName);
	MessageLog.Error(Text);
#endif
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FStormSyncDrivesModule, StormSyncDrives)
