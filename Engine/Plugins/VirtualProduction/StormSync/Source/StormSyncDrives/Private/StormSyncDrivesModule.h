// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncDrivesModule.h"
#include "UObject/UnrealType.h"

#include "StormSyncDrivesSettings.h"

#if WITH_EDITOR
#include "IMessageLogListing.h"
#endif

struct FStormSyncMountPointConfig;
class UStormSyncDrivesSettings;

/** Main entry point and implementation of StormSync Mounted Drives Runtime module. */
class FStormSyncDrivesModule : public IStormSyncDrivesModule
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ Begin IStormSyncDrivesModule interface
	virtual bool RegisterMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode = nullptr) override;
	virtual bool UnregisterMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText) override;
	//~ End IStormSyncDrivesModule interface

private:
	/** The name of the message output log we will send messages to */
	static constexpr const TCHAR* LogName = TEXT("StormSyncDrives");

	/**
	 * Stores a cache of the currently mounted paths. This will allow us to determine when the user changes settings
	 * and require us to update our mount mapping. 
	 */
	TMap<FString, FStormSyncMountPointConfig> CachedMountPoints;

#if WITH_EDITORONLY_DATA
	/** Listing used in the editor by the Message Log. */
	TSharedPtr<IMessageLogListing> LogListing;
#endif

	/** Core OnPostEngineInit delegate used to mount any stored config in developer settings */
	void OnPostEngineInit();

	/** Return the storm sync mount point config that was set on the command line. */
	TArray<FStormSyncMountPointConfig> GetCommandLineConfigs() const;

	/** If the cache configs are not in the given list then unregister and remove them from the cache. */
	bool RemoveFromCacheIfNotInConfigs(const TArray<FStormSyncMountPointConfig>& Configs);

	/**
	 * Developer settings changed handler. Called anytime settings changed.
	 *
	 * Used to validate user config and if successful, mount the set of Mount Points to Mount Directories.
	 */
	void OnSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InPropertyChangedEvent);

	/**
	 * Checks to see if the given mount point is currently in the cache.  If yes, then that implies that
	 * that we have already mapped the mount point and we do not need to re-process.
	 */
	bool IsMountPointInCache(const FStormSyncMountPointConfig& InMountPoint) const;

	/**
	 * Helper function that unregisters an existing mount point and remaps it.
	 */
	bool MountDriveHelper(const FStormSyncMountPointConfig& InMountPoint);

	/**
	 * Unregisters previously mounted drives, cache (update internal list of MountedDrives shared pointers),
	 * and register / mount the new config.
	 * 
	 * Happens once on engine init, and anytime the user settings changed.
	 */
	void ResetMountedDrivesFromSettings(const UStormSyncDrivesSettings* InSettings);

	void UnregisterMountedDrives();
	
	/** Static helper to log an error to the message log. Used during validation process. */
	static void AddMessageError(const FText& Text);
};
