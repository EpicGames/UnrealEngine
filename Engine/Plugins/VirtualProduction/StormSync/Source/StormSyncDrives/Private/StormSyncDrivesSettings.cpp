// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncDrivesSettings.h"

UStormSyncDrivesSettings::UStormSyncDrivesSettings()
{
	CategoryName = TEXT("Storm Sync");
}
