// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncDrivesUtils.h"

#include "HAL/FileManager.h"
#include "Misc/ScopeExit.h"
#include "StormSyncDrivesEnums.h"
#include "StormSyncDrivesSettings.h"

#define LOCTEXT_NAMESPACE "FStormSyncDrivesUtils"

bool FStormSyncDrivesUtils::ValidateMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode)
{
	if (!ValidateRootPath(InMountPoint.MountPoint, ErrorText, OutErrorCode))
	{
		return false;
	}

	if (!ValidateDirectory(InMountPoint.MountDirectory, ErrorText, OutErrorCode))
	{
		return false;
	}

	return true;
}

bool FStormSyncDrivesUtils::ValidateRootPath(const FString& InRootPath, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode)
{
	EStormSyncDriveErrorCode ErrorCode = EStormSyncDriveErrorCode::None;
	ON_SCOPE_EXIT
	{
		if (OutErrorCode)
		{
			*OutErrorCode = ErrorCode;
		}
	};

	// Non empty
	if (InRootPath.IsEmpty())
	{
		ErrorText = LOCTEXT("ValidationError_Empty_RootPath", "Root Path is empty");
		ErrorCode = EStormSyncDriveErrorCode::EmptyRootPath;
		return false;
	}

	// No mounting allowed at top lvl root
	if (InRootPath == TEXT("/"))
	{
		ErrorText = LOCTEXT("ValidationError_SingleSlash", "Cannot mount a directory on the \"/\" Root Path");
		ErrorCode = EStormSyncDriveErrorCode::TopLevelRootDisallowed;
		return false;
	}

	// Test validity by checking if it is a standard package name
	FPackageName::EErrorCode Reason;
	if (!IsValidTextForLongPackageName(InRootPath, &Reason))
	{
		ErrorText = FPackageName::FormatErrorAsText(InRootPath, Reason);
		ErrorCode = EStormSyncDriveErrorCode::InvalidPackageNameFormat;
		return false;
	}

	// Test for one lvl only (only allowed within /Game or an existing plugin content path)
	// Right now, only allow if within /Game (no mounting within plugins)
	//
	// Outside of an already existing mount point, engine will add the mount point but fail to find any assets within it
	// eg. /Foo/Bar would appear in content browser but Bar would appear empty
	if (!InRootPath.StartsWith(TEXT("/Game")) && !IsValidRootPathLevel(InRootPath))
	{
		ErrorText = FText::Format(
			LOCTEXT("ValidationError_RootPath_Level", "\"{0}\" is not a valid root path. It must be one level only (eg. /Example) when not within /Game."),
			FText::FromString(InRootPath)
		);
		ErrorCode = EStormSyncDriveErrorCode::InvalidPathLevel;
		return false;
	}

	return true;
}

bool FStormSyncDrivesUtils::ValidateDirectory(const FDirectoryPath& InDirectory, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode)
{
	EStormSyncDriveErrorCode ErrorCode = EStormSyncDriveErrorCode::None;
	ON_SCOPE_EXIT
	{
		if (OutErrorCode)
		{
			*OutErrorCode = ErrorCode;
		}
	};

	if (InDirectory.Path.IsEmpty())
	{
		ErrorText = LOCTEXT("ValidationError_Empty_Directory", "Directory Path is empty");
		ErrorCode = EStormSyncDriveErrorCode::EmptyDirectoryPath;
		return false;
	}

	if (!IFileManager::Get().DirectoryExists(*InDirectory.Path))
	{
		ErrorText = FText::Format(LOCTEXT("ValidationError_Invalid_Directory", "Directory '{0}' does not exist."), FText::FromString(InDirectory.Path));
		ErrorCode = EStormSyncDriveErrorCode::DirectoryNotFound;
		return false;
	}

	return true;
}

bool FStormSyncDrivesUtils::ValidateNonDuplicates(const TArray<FStormSyncMountPointConfig> InMountPoints, TArray<FText>& ValidationErrors)
{
	bool bResult = true;

	TArray<FString> RootPaths;
	TArray<FString> MountDirectories;

	int32 Index = 0;
	for (const FStormSyncMountPointConfig& Entry : InMountPoints)
	{
		const FText FormatText = LOCTEXT("ValidationError_Duplicate", "MountPoint at index {0} contains a duplicate entry (MountPoint: {1}, MountDirectory: {2}) - {3}");

		// Test for Mount Points duplicates
		if (RootPaths.Contains(Entry.MountPoint))
		{
			bResult = false;
			const FText ErrorText = LOCTEXT("ValidationError_RootPaths_Duplicate", "Mount Point values must be unique.");
			ValidationErrors.Add(FText::Format(
				FormatText,
				FText::AsNumber(Index),
				FText::FromString(Entry.MountPoint),
				FText::FromString(Entry.MountDirectory.Path),
				ErrorText
			));
		}

		// Test for Mount Directories duplicates
		if (MountDirectories.Contains(Entry.MountDirectory.Path))
		{
			bResult = false;
			const FText ErrorText = LOCTEXT("ValidationError_Directory_Duplicate", "Mount Directory values must be unique.");
			ValidationErrors.Add(FText::Format(
				FormatText,
				FText::AsNumber(Index),
				FText::FromString(Entry.MountPoint),
				FText::FromString(Entry.MountDirectory.Path),
				ErrorText
			));
		}

		RootPaths.AddUnique(Entry.MountPoint);
		MountDirectories.AddUnique(Entry.MountDirectory.Path);

		Index++;
	}

	return bResult;
}

bool FStormSyncDrivesUtils::IsValidRootPathLevel(const FString& InRootPath)
{
	FString RootPath = InRootPath;
	RootPath.RemoveFromStart(TEXT("/"));
	RootPath.RemoveFromEnd(TEXT("/"));

	TArray<FString> Splits;
	RootPath.ParseIntoArray(Splits, TEXT("/"));

	// Must be exactly one level
	return Splits.Num() == 1;
}

bool FStormSyncDrivesUtils::IsValidTextForLongPackageName(FStringView InLongPackageName, FPackageName::EErrorCode* OutReason)
{
	// All package names must contain a leading slash, root, slash and name, at minimum theoretical length ("/A/B") is 4
	// Only difference with FPackageName::IsValidTextForLongPackageName is that we don't test for minimum theoretical length

	// Package names start with a leading slash.
	if (InLongPackageName[0] != '/')
	{
		if (OutReason) *OutReason = FPackageName::EErrorCode::LongPackageNames_PathWithNoStartingSlash;
		return false;
	}
	// Package names do not end with a trailing slash.
	if (InLongPackageName[InLongPackageName.Len() - 1] == '/')
	{
		if (OutReason) *OutReason = FPackageName::EErrorCode::LongPackageNames_PathWithTrailingSlash;
		return false;
	}
	if (InLongPackageName.Contains(TEXT("//")))
	{
		if (OutReason) *OutReason = FPackageName::EErrorCode::LongPackageNames_PathWithDoubleSlash;
		return false;
	}
	// Check for invalid characters
	if (FPackageName::DoesPackageNameContainInvalidCharacters(InLongPackageName, OutReason))
	{
		return false;
	}
	if (OutReason) *OutReason = FPackageName::EErrorCode::PackageNameUnknown;
	return true;
}


#undef LOCTEXT_NAMESPACE
