// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/FileManager.h"
#include "IStormSyncDrivesModule.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "StormSyncDrivesEnums.h"
#include "StormSyncDrivesSettings.h"

BEGIN_DEFINE_SPEC(FStormSyncDrivesModuleSpec, "StormSync.StormSyncDrives", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	FString TestMountPoint;
	FString TestContentPath;
	FString InvalidDirectoryPath;

END_DEFINE_SPEC(FStormSyncDrivesModuleSpec)

void FStormSyncDrivesModuleSpec::Define()
{
	TestMountPoint = TEXT("/Foo");
	TestContentPath = FPaths::ProjectSavedDir() / TEXT("StormSyncTests");
	InvalidDirectoryPath = FPaths::ProjectSavedDir() / TEXT("MostLikelyNonExistingDirectory");

	Describe(TEXT("IStormSyncDrivesModule"), [this]()
	{
		BeforeEach([this]()
		{
			if (!IFileManager::Get().DirectoryExists(*TestContentPath))
			{
				if (!IFileManager::Get().MakeDirectory(*TestContentPath))
				{
					AddError(FString::Printf(TEXT("Failed to create directory %s for automated tests"), *TestContentPath));
				}
			}
		});

		Describe(TEXT("IStormSyncDrivesModule::RegisterMountPoint()"), [this]()
		{
			It(TEXT("should fail on invalid input - empty mount point"), [this]()
			{
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TEXT("");

				// Two times, once for message log, once for error log
				const FString ExpectedErrorText = TEXT("Root Path is empty");
				AddExpectedError(ExpectedErrorText, EAutomationExpectedErrorFlags::Contains, 1);

				FText ErrorText;
				EStormSyncDriveErrorCode ErrorCode = EStormSyncDriveErrorCode::None;

				const bool bSuccess = IStormSyncDrivesModule::Get().RegisterMountPoint(MountPoint, ErrorText, &ErrorCode);
				TestFalse(TEXT("Expected return value is false"), bSuccess);
				TestTrue(TEXT("ErrorText expected value"), ErrorCode == EStormSyncDriveErrorCode::EmptyRootPath);
			});

			It(TEXT("should fail on invalid input - empty directory"), [this]()
			{
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TestMountPoint;
				MountPoint.MountDirectory.Path = TEXT("");

				const FString ExpectedErrorText = TEXT("Directory Path is empty");
				AddExpectedError(ExpectedErrorText, EAutomationExpectedErrorFlags::Contains, 1);

				FText ErrorText;
				EStormSyncDriveErrorCode ErrorCode = EStormSyncDriveErrorCode::None;

				const bool bSuccess = IStormSyncDrivesModule::Get().RegisterMountPoint(MountPoint, ErrorText, &ErrorCode);
				TestFalse(TEXT("Expected return value is false"), bSuccess);
				TestTrue(TEXT("ErrorText expected value"), ErrorCode == EStormSyncDriveErrorCode::EmptyDirectoryPath);
			});

			It(TEXT("should fail on invalid input - invalid directory"), [this]()
			{
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TestMountPoint;
				MountPoint.MountDirectory.Path = InvalidDirectoryPath;

				const FString ExpectedErrorText = FString::Printf(TEXT("Directory '%s' does not exist."), *InvalidDirectoryPath);
				AddExpectedError(ExpectedErrorText, EAutomationExpectedErrorFlags::Contains, 1);

				FText ErrorText;
				EStormSyncDriveErrorCode ErrorCode = EStormSyncDriveErrorCode::None;

				const bool bSuccess = IStormSyncDrivesModule::Get().RegisterMountPoint(MountPoint, ErrorText, &ErrorCode);
				TestFalse(TEXT("Expected return value is false"), bSuccess);
				TestTrue(TEXT("ErrorText expected value"), ErrorCode == EStormSyncDriveErrorCode::DirectoryNotFound);
			});

			It(TEXT("should mount directory with valid configuration"), [this]()
			{
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TestMountPoint;
				MountPoint.MountDirectory.Path = TestContentPath;

				// Mount points are always added with a trailing slash (eg. /Foo => /Foo/)
				const FString CheckMountPoint = TestMountPoint + TEXT("/");

				TestFalse(TEXT("Path is not mounted yet"), FPackageName::MountPointExists(CheckMountPoint));
				FText ErrorText;
				const bool bSuccess = IStormSyncDrivesModule::Get().RegisterMountPoint(MountPoint, ErrorText);
				TestTrue(TEXT("Expected return value is true"), bSuccess);
				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());
				TestTrue(TEXT("Path is now mounted"), FPackageName::MountPointExists(CheckMountPoint));
			});
		});

		Describe(TEXT("IStormSyncDrivesModule::UnregisterMountPoint()"), [this]()
		{
			It(TEXT("should unregister previously added mount point"), [this]()
			{
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TestMountPoint;
				MountPoint.MountDirectory.Path = TestContentPath;

				// Mount points are always added with a trailing slash (eg. /Foo => /Foo/)
				const FString CheckMountPoint = TestMountPoint + TEXT("/");

				TestFalse(TEXT("Path is not mounted yet"), FPackageName::MountPointExists(CheckMountPoint));

				FText ErrorText;
				const bool bRegisterMountPoint = IStormSyncDrivesModule::Get().RegisterMountPoint(MountPoint, ErrorText);
				TestTrue(TEXT("Expected return bRegisterMountPoint is true"), bRegisterMountPoint);
				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());
				TestTrue(TEXT("Path is now mounted"), FPackageName::MountPointExists(CheckMountPoint));

				const bool bUnregisterMountPoint = IStormSyncDrivesModule::Get().UnregisterMountPoint(MountPoint, ErrorText);
				TestTrue(TEXT("Expected return bUnregisterMountPoint is true"), bUnregisterMountPoint);
				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());
				TestFalse(TEXT("Path is now unmounted"), FPackageName::MountPointExists(CheckMountPoint));
			});
		});


		AfterEach([this]()
		{
			// Mount points are always added with a trailing slash (eg. /Foo => /Foo/)
			const FString CheckMountPoint = TestMountPoint + TEXT("/");

			if (FPackageName::MountPointExists(CheckMountPoint))
			{
				FPackageName::UnRegisterMountPoint(CheckMountPoint, TestContentPath);
			}

			if (IFileManager::Get().DirectoryExists(*TestContentPath))
			{
				if (!IFileManager::Get().DeleteDirectory(*TestContentPath))
				{
					AddError(FString::Printf(TEXT("Failed to delete directory %s for automated tests"), *TestContentPath));
				}
			}
		});
	});
}
