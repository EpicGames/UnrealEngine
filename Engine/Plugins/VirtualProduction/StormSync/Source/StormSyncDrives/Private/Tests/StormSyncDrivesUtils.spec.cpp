// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "StormSyncDrivesSettings.h"
#include "StormSyncDrivesUtils.h"

BEGIN_DEFINE_SPEC(FStormSyncDrivesUtilsSpec, "StormSync.StormSyncDrives", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	/** Helper to assert a duplicate error log */
	void AssertErrorLog(const int32 InErrorIndex, const int32 InTestCaseIndex, const FStormSyncMountPointConfig& InTestCase, const TArray<FText>& InValidationErrors)
	{
		if (!InValidationErrors.IsValidIndex(InErrorIndex))
		{
			AddError(FString::Printf(TEXT("Invalid index")));
			return;
		}

		const FString ExpectedLog = FString::Printf(
			TEXT("MountPoint at index %d contains a duplicate entry (MountPoint: %s, MountDirectory: %s) - %s"),
			InTestCaseIndex,
			*InTestCase.MountPoint,
			*InTestCase.MountDirectory.Path,
			TEXT("Mount Point values must be unique.")
		);
		TestEqual(TEXT("Expected error log"), InValidationErrors[InErrorIndex].ToString(), ExpectedLog);
	}

END_DEFINE_SPEC(FStormSyncDrivesUtilsSpec)

void FStormSyncDrivesUtilsSpec::Define()
{
	Describe(TEXT("FStormSyncDrivesUtils"), [this]()
	{
		Describe(TEXT("ValidateRootPath"), [this]()
		{
			It(TEXT("should fail on empty path"), [this]()
			{
				FText ErrorText;
				const bool bResult = FStormSyncDrivesUtils::ValidateRootPath(TEXT(""), ErrorText);

				TestFalse(TEXT("ValidateRootPath status NOK"), bResult);
				TestEqual(TEXT("ErrorText expected"), ErrorText.ToString(), TEXT("Root Path is empty"));
			});

			It(TEXT("should fail on / path"), [this]()
			{
				FText ErrorText;
				const bool bResult = FStormSyncDrivesUtils::ValidateRootPath(TEXT("/"), ErrorText);

				TestFalse(TEXT("ValidateRootPath status NOK"), bResult);
				TestEqual(TEXT("ErrorText expected"), ErrorText.ToString(), TEXT("Cannot mount a directory on the \"/\" Root Path"));
			});

			It(TEXT("should fail without a leading slash"), [this]()
			{
				TArray<FString> TestCases = {
					TEXT("Foo"),
					TEXT("Foo/Bar"),
					TEXT(" Foo"),
					TEXT(" Foo/Bar"),
				};

				for (const FString& TestCase : TestCases)
				{
					FText ErrorText;
					const bool bResult = FStormSyncDrivesUtils::ValidateRootPath(TestCase, ErrorText);

					TestFalse(TEXT("ValidateRootPath status NOK"), bResult);
					TestEqual(
						TEXT("ErrorText expected"),
						ErrorText.ToString(),
						FString::Printf(TEXT("Input '%s' does not start with a '/', which is required for LongPackageNames."), *TestCase)
					);
				}
			});

			It(TEXT("should fail if ending with a trailing slash"), [this]()
			{
				FText ErrorText;
				const FString TestPath = TEXT("/Foo/");
				const bool bResult = FStormSyncDrivesUtils::ValidateRootPath(TestPath, ErrorText);

				TestFalse(TEXT("ValidateRootPath status NOK"), bResult);
				TestEqual(
					TEXT("ErrorText expected"),
					ErrorText.ToString(),
					FString::Printf(TEXT("Input '%s' ends with a '/', which is invalid for LongPackageNames."), *TestPath)
				);
			});

			It(TEXT("should fail if path has double slash"), [this]()
			{
				TArray<FString> TestCases = {
					TEXT("/Foo//Bar"),
					TEXT("//Foo/Bar"),
					TEXT("//Foo//Bar"),
				};

				for (const FString& TestCase : TestCases)
				{
					FText ErrorText;
					const bool bResult = FStormSyncDrivesUtils::ValidateRootPath(TestCase, ErrorText);

					TestFalse(TEXT("ValidateRootPath status NOK"), bResult);
					TestEqual(
						TEXT("ErrorText expected"),
						ErrorText.ToString(),
						FString::Printf(TEXT("Input '%s' contains '//', which is invalid for LongPackageNames."), *TestCase)
					);
				}
			});

			It(TEXT("should fail if path has invalid characters"), [this]()
			{
				TArray<FString> TestCases = {
					TEXT("/Foo\\Bar"),
					TEXT("/Foo:Bar"),
					TEXT("/Foo*Bar"),
					TEXT("/Foo?Bar"),
					TEXT("/Foo\"Bar"),
					TEXT("/Foo<Bar"),
					TEXT("/Foo>Bar"),
					TEXT("/Foo|Bar"),
					TEXT("/Foo'Bar"),
					TEXT("/Foo Bar"),
					TEXT("/Foo,Bar"),
					TEXT("/Foo.Bar"),
					TEXT("/Foo&Bar"),
					TEXT("/Foo&Bar"),
					TEXT("/Foo!Bar"),
					TEXT("/Foo~Bar"),
					TEXT("/Foo\nBar"),
					TEXT("/Foo\rBar"),
					TEXT("/Foo\tBar"),
					TEXT("/Foo@Bar"),
					TEXT("/Foo#Bar"),
				};

				const FString IllegalNameCharacters = FString(INVALID_LONGPACKAGE_CHARACTERS);

				for (const FString& TestCase : TestCases)
				{
					FText ErrorText;
					const bool bResult = FStormSyncDrivesUtils::ValidateRootPath(TestCase, ErrorText);

					TestFalse(TEXT("ValidateRootPath status NOK"), bResult);
					TestEqual(
						TEXT("ErrorText expected"),
						ErrorText.ToString(),
						FString::Printf(TEXT("Input '%s' contains one of the invalid characters for LongPackageNames: '%s'."), *TestCase, *IllegalNameCharacters)
					);
				}
			});

			It(TEXT("should return no error with valid path"), [this]()
			{
				TArray<FString> TestCases = {
					TEXT("/Game"),
					TEXT("/Foo"),
					TEXT("/Game/Foo"),
					TEXT("/Game/Foo/Bar"),
					TEXT("/Game/Foo/Bar/Baz"),
				};

				for (const FString& TestCase : TestCases)
				{
					FText ErrorText;
					const bool bResult = FStormSyncDrivesUtils::ValidateRootPath(TestCase, ErrorText);

					TestTrue(TEXT("ValidateRootPath status OK"), bResult);
					TestTrue(TEXT("Error Text is empty"), ErrorText.IsEmpty());
				}
			});
			
			It(TEXT("should allow only one level package"), [this]()
			{
				TArray<FString> ValidCases = {
					TEXT("/Foo/"),
					TEXT("/Foo"),
				};

				TArray<FString> InvalidCases = {
					TEXT(""),
					TEXT("/"),
					TEXT("/Foo/Bar"),
					TEXT("/Foo/Bar/"),
					TEXT("/Foo/Bar/Baz"),
					TEXT("/Game/Foo/Bar/Baz"),
					TEXT("/Game/Foo/Bar/Baz/"),
				};

				for (const FString& TestCase : ValidCases)
				{
					TestTrue(
						FString::Printf(TEXT("IsValidRootPathLevel returns true for %s"), *TestCase),
						FStormSyncDrivesUtils::IsValidRootPathLevel(TestCase)
					);
				}

				for (const FString& TestCase : InvalidCases)
				{
					TestFalse(
						FString::Printf(TEXT("IsValidRootPathLevel returns false for %s"), *TestCase),
						FStormSyncDrivesUtils::IsValidRootPathLevel(TestCase)
					);
				}
			});
		});

		Describe(TEXT("ValidateDirectory"), [this]()
		{
			It(TEXT("should fail with empty path"), [this]()
			{
				FText ErrorText;
				FDirectoryPath Path;
				Path.Path = TEXT("");
				const bool bSuccess = FStormSyncDrivesUtils::ValidateDirectory(Path, ErrorText);

				TestFalse(TEXT("ValidateDirectory status NOK"), bSuccess);
				TestEqual(TEXT("ErrorText expected"), ErrorText.ToString(),TEXT("Directory Path is empty"));
			});

			It(TEXT("should fail with non existing directory"), [this]()
			{
				FText ErrorText;
				FDirectoryPath Path;
				Path.Path = FPaths::ProjectSavedDir() / TEXT("MostLikelyNonExistingDirectory");
				const bool bSuccess = FStormSyncDrivesUtils::ValidateDirectory(Path, ErrorText);

				TestFalse(TEXT("ValidateDirectory status NOK"), bSuccess);
				TestEqual(
					TEXT("ErrorText expected"),
					ErrorText.ToString(),
					FString::Printf(TEXT("Directory '%s' does not exist."), *Path.Path)
				);
			});

			It(TEXT("should return no error with valid and existing directory"), [this]()
			{
				FText ErrorText;
				FDirectoryPath Path;
				Path.Path = FPaths::ProjectSavedDir();
				const bool bSuccess = FStormSyncDrivesUtils::ValidateDirectory(Path, ErrorText);

				TestTrue(TEXT("ValidateDirectory status NOK"), bSuccess);
				TestTrue(TEXT("Error Text is empty"), ErrorText.IsEmpty());
			});
		});

		Describe(TEXT("ValidateMountPoint"), [this]()
		{
			It(TEXT("should return no error with a proper mount point config"), [this]()
			{
				FText ErrorText;
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TEXT("/Foo");
				MountPoint.MountDirectory.Path = FPaths::ProjectSavedDir();
				const bool bResult = FStormSyncDrivesUtils::ValidateMountPoint(MountPoint, ErrorText);

				TestTrue(TEXT("ValidateMountPoint status OK"), bResult);
				TestTrue(TEXT("Error Text is empty"), ErrorText.IsEmpty());
			});

			It(TEXT("should fail if root path is invalid"), [this]()
			{
				FText ErrorText;
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TEXT("");
				MountPoint.MountDirectory.Path = FPaths::ProjectSavedDir();
				const bool bResult = FStormSyncDrivesUtils::ValidateMountPoint(MountPoint, ErrorText);

				TestFalse(TEXT("ValidateMountPoint status NOK"), bResult);
				// More extensive error testing is done by ValidateRootPath spec
				TestFalse(TEXT("Error Text is not empty"), ErrorText.IsEmpty());
			});

			It(TEXT("should fail if root directory is invalid"), [this]()
			{
				FText ErrorText;
				FStormSyncMountPointConfig MountPoint;
				MountPoint.MountPoint = TEXT("/Foo");
				MountPoint.MountDirectory.Path = FPaths::ProjectSavedDir() / TEXT("MostLikelyNonExistingDirectory");
				const bool bResult = FStormSyncDrivesUtils::ValidateMountPoint(MountPoint, ErrorText);

				TestFalse(TEXT("ValidateMountPoint status NOK"), bResult);
				// More extensive error testing is done by ValidateRootPath spec
				TestFalse(TEXT("Error Text is not empty"), ErrorText.IsEmpty());
			});
		});

		Describe(TEXT("ValidateNonDuplicates"), [this]()
		{
			It(TEXT("should return no error with empty array"), [this]()
			{
				TArray<FText> ValidationErrors;
				const TArray<FStormSyncMountPointConfig> MountPoints;
				const bool bResult = FStormSyncDrivesUtils::ValidateNonDuplicates(MountPoints, ValidationErrors);

				TestTrue(TEXT("ValidateNonDuplicates status OK"), bResult);
				TestTrue(TEXT("ValidationErrors is empty"), ValidationErrors.IsEmpty());
			});

			It(TEXT("should return no error with no duplicates"), [this]()
			{
				const TArray MountPoints = {
					FStormSyncMountPointConfig(TEXT("/Foo"), FPaths::ProjectSavedDir() / TEXT("Path_01")),
					FStormSyncMountPointConfig(TEXT("/Bar"), FPaths::ProjectSavedDir() / TEXT("Path_02")),
					FStormSyncMountPointConfig(TEXT("/Baz"), FPaths::ProjectSavedDir() / TEXT("Path_03"))
				};

				TArray<FText> ValidationErrors;
				const bool bResult = FStormSyncDrivesUtils::ValidateNonDuplicates(MountPoints, ValidationErrors);

				TestTrue(TEXT("ValidateNonDuplicates status OK"), bResult);
				TestTrue(TEXT("ValidationErrors is empty"), ValidationErrors.IsEmpty());
			});

			It(TEXT("should return errors with mount point duplicates"), [this]()
			{
				const TArray MountPoints = {
					FStormSyncMountPointConfig(TEXT("/Foo"), FPaths::ProjectSavedDir() / TEXT("Path_01")),
					FStormSyncMountPointConfig(TEXT("/Foo"), FPaths::ProjectSavedDir() / TEXT("Path_02")),
					FStormSyncMountPointConfig(TEXT("/Baz"), FPaths::ProjectSavedDir() / TEXT("Path_03")),
					FStormSyncMountPointConfig(TEXT("/Foo"), FPaths::ProjectSavedDir() / TEXT("Path_04"))
				};

				TArray<FText> ValidationErrors;
				const bool bResult = FStormSyncDrivesUtils::ValidateNonDuplicates(MountPoints, ValidationErrors);

				TestFalse(TEXT("ValidateNonDuplicates status NOK"), bResult);
				TestEqual(TEXT("ValidationErrors is of expected size"), ValidationErrors.Num(), 2);

				AssertErrorLog(0, 1, MountPoints[1], ValidationErrors);
				AssertErrorLog(1, 3, MountPoints[3], ValidationErrors);
			});
		});
	});
}
