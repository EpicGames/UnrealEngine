// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

enum class EStormSyncDriveErrorCode : uint8;
struct FStormSyncMountPointConfig;

/** Public interface for Storm Sync Drives module and API */
class IStormSyncDrivesModule : public IModuleInterface
{
public:
	static bool IsAvailable()
	{
		static const FName ModuleName = TEXT("StormSyncDrives");
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IStormSyncDrivesModule& Get()
	{
		static const FName ModuleName = TEXT("StormSyncDrives");
		return FModuleManager::LoadModuleChecked<IStormSyncDrivesModule>(ModuleName);
	}

	/**
	 * Register and mount the provided mount point config (with a Mount Point root path and Mount Directory filesystem path)
	 *
	 * Mount root path and directory will be validated first, and errors will be logged into Message Logs.
	 *
	 * @param InMountPoint The config with root mount path & directory to mount
	 * @param ErrorText Localized text describing the error in case mounting failed
	 * @param OutError optional enum for further details on the operation's error if any
	 * @returns true if we were able to register the mount point
	 */
	virtual bool RegisterMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText, EStormSyncDriveErrorCode* OutError = nullptr) = 0;

	/**
	 * Unregisters the provided mount point config, if it was previously mounted.
	 * 
	 * @param InMountPoint The config with root mount path & directory to unmount
	 * @param ErrorText Localized text describing the error in case unmounting failed
	 * 
	 * @returns true if we were able to unregister the mount point
	 */
	virtual bool UnregisterMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText) = 0;
};
