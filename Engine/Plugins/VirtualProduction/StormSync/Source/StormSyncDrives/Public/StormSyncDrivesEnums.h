// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

enum class EStormSyncDriveErrorCode : uint8
{
	/** No error */
	None,

	/** Root path is empty */
	EmptyRootPath,

	/** Path is "/". No mounting allowed at the top level root */
	TopLevelRootDisallowed,

	/** Path does not have a standard package name format */
	InvalidPackageNameFormat,

	/** Path is expected to be within /Game or one level path e.g. /Example when not within /Game */
	InvalidPathLevel,

	/** Directory path is empty */
	EmptyDirectoryPath,

	/** Directory path does not exist */
	DirectoryNotFound,
};
