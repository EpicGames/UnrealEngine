// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "StormSyncDrivesSettings.generated.h"

/** Holds config for a given Mount Point (eg. /RootPath) to a filesystem directory (can be local, could be networked) */
USTRUCT()
struct FStormSyncMountPointConfig
{
	GENERATED_BODY()

	/** Represents the root Mount Point for the drive (eg. /MyDrive or /Game/MyDrive) */
	UPROPERTY(EditAnywhere, Category="Mounted Drives")
	FString MountPoint;

	/** Represents the absolute path of the mounted directory */
	UPROPERTY(EditAnywhere, Category="Mounted Drives")
	FDirectoryPath MountDirectory;

	/** The path in MountDirectory should be interpreted as a relative path to the content directory of the active project. */
	UPROPERTY(EditAnywhere, Category = "Mounted Drives")
	bool bResolveAsRelativePath = false;
	
	FStormSyncMountPointConfig() = default;

	FStormSyncMountPointConfig(const FString& InMountPoint, const FDirectoryPath& InMountDirectory, bool bInResolveAsRelativePath = false)
		: MountPoint(InMountPoint)
		, MountDirectory(InMountDirectory)
		, bResolveAsRelativePath(bInResolveAsRelativePath)
	{
	}

	FStormSyncMountPointConfig(const FString& InMountPoint, const FString& InMountDirectory, bool bInResolveAsRelativePath = false)
		: MountPoint(InMountPoint)
		, bResolveAsRelativePath(bInResolveAsRelativePath)
	{
		MountDirectory.Path = InMountDirectory;
	}
};

/**
 * Settings for the StormSyncDrives module.
 *
 * Handle configuration for shared network or local mounted drives.
 *
 * Configuration must adhere to the following rules:
 * 
 * * Should not have multiple entries with same mount point and / or directory (Mount Paths / Directories must have unique values)
 * * Mount Points and Directories must not be empty
 * * Mount Point cannot be "/", must start with a leading "/" and should not end with a trailing "/"
 * * Mount Point cannot contain "//" or any other invalid characters (\\:*?\"<*|' ,.&!~\n\r\t@#)
 * * Mount Point when outside "/Game" must be one level only (eg. /Example is ok, /Example/Path is not)
 * * Mount Directory must match an existing filesystem path
 *
 * Changes done here will be validated and errors logged to message log.
 *
 * If successful, mount points will be updated and should be available in the content browser.
 */
UCLASS(MinimalAPI, Config=Game, DefaultConfig, DisplayName="Mount Points Settings")
class UStormSyncDrivesSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UStormSyncDrivesSettings();

	/** List of mounted drives configuration */
	UPROPERTY(config, EditAnywhere, Category = "Mounted Drives", meta=(TitleProperty="{MountPoint} mapped to {MountDirectory}"))
	TArray<FStormSyncMountPointConfig> MountPoints;
};
