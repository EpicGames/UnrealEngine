// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Misc/PackageName.h"

enum class EStormSyncDriveErrorCode : uint8;
struct FStormSyncMountPointConfig;

/** Provides validation helpers */
class STORMSYNCDRIVES_API FStormSyncDrivesUtils
{
public:
	/** Checks validity for a mount point */
	static bool ValidateMountPoint(const FStormSyncMountPointConfig& InMountPoint, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode = nullptr);

	/** Checks validity for a mount point */
	static bool ValidateRootPath(const FString& InRootPath, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode = nullptr);

	/** Checks validity for a mount directory */
	static bool ValidateDirectory(const FDirectoryPath& InDirectory, FText& ErrorText, EStormSyncDriveErrorCode* OutErrorCode = nullptr);

	/** Checks validity for any duplicates regarding Mount Point (Root Paths) and filesystem directory */
	static bool ValidateNonDuplicates(const TArray<FStormSyncMountPointConfig> InMountPoints, TArray<FText>& ValidationErrors);

	/** Checks validity of mount points and only allow one level packages (eg. /Foo again /Foo/Bar) */
	static bool IsValidRootPathLevel(const FString& InRootPath);

	/**
	 * Report whether a given name is the proper format for a PackageName, without checking whether it is in one of the registered mount points.
	 *
	 * Notable difference from FPackageName::IsValidTextForLongPackageName is that we don't check for minimal path length requirement,
	 * we are only interested in testing the front path (eg. /Foo vs /Foo/Dummy)
	 *
	 * @param InLongPackageName The package name to test
	 * @param OutReason When returning false, this will provide a description of what was wrong with the name.
	 * @return true if valid text for a long package name
	 */
	static bool IsValidTextForLongPackageName(FStringView InLongPackageName, FPackageName::EErrorCode* OutReason = nullptr);
};
