// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StormSyncDrives : ModuleRules
{
	public StormSyncDrives(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"Slate",
				"SlateCore",
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd",
				"MessageLog"
			});
		}
	}
}