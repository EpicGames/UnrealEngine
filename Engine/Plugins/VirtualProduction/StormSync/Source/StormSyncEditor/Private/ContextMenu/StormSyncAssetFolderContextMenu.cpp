// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncAssetFolderContextMenu.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetViewUtils.h"
#include "ContentBrowserAssetDataCore.h"
#include "ContentBrowserAssetDataPayload.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserDataSource.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/FileManager.h"
#include "IStormSyncTransportClientModule.h"
#include "IStormSyncTransportServerModule.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Slate/SStormSyncExportWizard.h"
#include "Slate/Status/SStormSyncStatusWidget.h"
#include "StormSyncCoreSettings.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncEditor.h"
#include "StormSyncEditorLog.h"
#include "StormSyncPackageDescriptor.h"
#include "Subsystems/StormSyncNotificationSubsystem.h"
#include "ToolMenus.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FStormSyncAssetFolderContextMenu"

void FStormSyncAssetFolderContextMenu::Initialize()
{
	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::Initialize - ...");
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &FStormSyncAssetFolderContextMenu::RegisterMenus));
}

void FStormSyncAssetFolderContextMenu::Shutdown()
{
	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::Shutdown - ...");
}

void FStormSyncAssetFolderContextMenu::BuildPushAssetsMenuSection(FMenuBuilder& InMenuBuilder, const TArray<FName> InPackageNames, const bool bInIsPushing)
{
	InMenuBuilder.BeginSection(TEXT("StormSyncActionsPushAssets"), LOCTEXT("StormSyncActionsPushAssetsMenuHeading", "Remotes"));
	BuildPushAssetsMenuEntries(InMenuBuilder, InPackageNames, bInIsPushing);
	InMenuBuilder.EndSection();
}

void FStormSyncAssetFolderContextMenu::BuildCompareWithMenuSection(FMenuBuilder& MenuBuilder, TArray<FName> InPackageNames) const
{
	MenuBuilder.BeginSection(TEXT("StormSyncActionsCompareWith"), LOCTEXT("StormSyncActionsCompareWithMenuHeading", "Remotes"));
	
	TMap<FMessageAddress, FStormSyncConnectedDevice> RegisteredConnections = FStormSyncEditorModule::Get().GetRegisteredConnections();

	// Deal with entry if there is no valid registered connections currently
	if (RegisteredConnections.IsEmpty())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NoRemotesConnected", "No valid connections currently detected on the network."),
			LOCTEXT("NoRemotesConnectedTooltip", "No valid connections currently detected on the network."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateStatic([]() { return false; })
			)
		);

		return;
	}

	// Deal with entries for each active registered connections
	for (const TPair<FMessageAddress, FStormSyncConnectedDevice>& RegisteredConnection : RegisteredConnections)
	{
		const FMessageAddress& MessageAddress = RegisteredConnection.Key;
		const FStormSyncConnectedDevice Connection = RegisteredConnection.Value;

		const FText RemoteDescription = FText::FromString(FString::Printf(TEXT("%s (%s)"), *Connection.HostName, *Connection.ProjectDir));
		FText LabelText = FText::Format(
			LOCTEXT("CompareAssetMenuAction", "Compare {0} selected asset(s) with {1}"),
			InPackageNames.Num(),
			RemoteDescription
		);

		const FText LabelTooltip = GetEntryTooltipForRemote(MessageAddress, Connection);

		FExecuteAction Delegate = FExecuteAction::CreateStatic(&FStormSyncAssetFolderContextMenu::ExecuteCompareWithAction, InPackageNames, Connection.StormSyncClientAddressId);

		MenuBuilder.AddMenuEntry(
			LabelText,
			LabelTooltip,
			FSlateIcon(),
			FUIAction(MoveTemp(Delegate))
		);
	}

	MenuBuilder.EndSection();	
}

TArray<FAssetData> FStormSyncAssetFolderContextMenu::GetDirtyAssets(const TArray<FName>& InPackageNames)
{
	TArray<FAssetData> AssetsData;
	AssetsData.Reserve(InPackageNames.Num());

	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	for (const FName& PackageName : InPackageNames)
	{
		TArray<FName> Dependencies;
		TArray<FAssetData> Assets;

		if (!AssetRegistry.GetAssetsByPackageName(PackageName, Assets))
		{
			UE_LOGF(LogStormSyncEditor, Warning, "FStormSyncAssetFolderContextMenu::HasDirtyAssets - GetAssetsByPackageName failed to load assets for %ls", *PackageName.ToString());
			continue;
		}

		// Filter out BlueprintGeneratedClass (for BPs, assets ending with _C suffix)
		TArray<FAssetData> ValidAssets = Assets.FilterByPredicate([](const FAssetData& AssetData)
		{
			const UObject* Asset = AssetData.GetAsset();
			return Asset && !Asset->IsA<UBlueprintGeneratedClass>();
		});

		AssetsData.Append(ValidAssets);
	}

	TArray<FAssetData> DirtyAssets;
	for (const FAssetData& AssetData : AssetsData)
	{
		const UPackage* Package = AssetData.GetPackage();
		if (Package && Package->IsDirty())
		{
			DirtyAssets.Add(AssetData);
		}
	}

	return DirtyAssets;
}

TArray<FAssetData> FStormSyncAssetFolderContextMenu::GetDirtyAssets(const TArray<FName>& InPackageNames, FText& OutDisabledReason)
{
	const TArray<FAssetData> DirtyAssets = GetDirtyAssets(InPackageNames);
	const bool bContainsDirtyAssets = !DirtyAssets.IsEmpty();
	
	if (bContainsDirtyAssets)
	{
		TArray<FText> DirtyAssetsTexts;
		Algo::Transform(DirtyAssets, DirtyAssetsTexts, [](const FAssetData& AssetData)
		{
			return FText::FromString(FString::Printf(TEXT("- %s"), *AssetData.PackageName.ToString()));
		});

		const FText DirtyListText = FText::Join(FText::FromString(LINE_TERMINATOR), DirtyAssetsTexts);
		OutDisabledReason = FText::Format(LOCTEXT("DisabledTooltipReason", "\n\nDisabled because of the following assets in dirty state (unsaved):\n\n{0}"), DirtyListText);
	}
	
	return DirtyAssets;
}

void FStormSyncAssetFolderContextMenu::RegisterMenus()
{
	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::RegisterMenus");

	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu"))
	{
		Menu->AddDynamicSection(TEXT("DynamicSection_StormSync_ContextMenu_Folder"), FNewToolMenuDelegate::CreateLambda([WeakThis = AsWeak()](UToolMenu* InMenu)
		{
			const TSharedPtr<FStormSyncAssetFolderContextMenu, ESPMode::ThreadSafe> This = WeakThis.Pin();
			if (This.IsValid())
			{
				This->PopulateAssetFolderContextMenu(InMenu);
			}
		}));
	}

	if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu"))
	{
		Menu->AddDynamicSection(TEXT("DynamicSection_StormSync_ContextMenu_Asset"), FNewToolMenuDelegate::CreateLambda([WeakThis = AsWeak()](UToolMenu* InMenu)
		{
			const TSharedPtr<FStormSyncAssetFolderContextMenu, ESPMode::ThreadSafe> This = WeakThis.Pin();
			if (This.IsValid())
			{
				This->PopulateAssetFileContextMenu(InMenu);
			}
		}));
	}
}

void FStormSyncAssetFolderContextMenu::PopulateAssetFolderContextMenu(UToolMenu* InMenu) const
{
	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::PopulateAssetFolderContextMenu - InMenu: %ls", *GetNameSafe(InMenu));
	check(InMenu);

	const UContentBrowserDataMenuContext_FolderMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_FolderMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_FolderMenu was missing!"));

	// Extract the internal package paths that belong to this data source from the full list of selected items given in the context
	TArray<FString> SelectedPackagePaths;
	TArray<FString> SelectedAssetPackages;
	GetSelectedFilesAndFolders(InMenu, SelectedPackagePaths, SelectedAssetPackages);

	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::PopulateAssetFolderContextMenu - Result ...");
	UE_LOGF(LogStormSyncEditor, Verbose, "\tSelectedPackagePaths: %d", SelectedPackagePaths.Num());
	for (const FString& SelectedPackagePath : SelectedPackagePaths)
	{
		UE_LOGF(LogStormSyncEditor, Verbose, "\t- %ls", *SelectedPackagePath);
	}

	UE_LOGF(LogStormSyncEditor, Verbose, "\tSelectedAssetPackages: %d", SelectedAssetPackages.Num());
	for (const FString& SelectedAssetPackage : SelectedAssetPackages)
	{
		UE_LOGF(LogStormSyncEditor, Verbose, "\t- %ls", *SelectedAssetPackage);
	}
	
	AddFolderMenuOptions(InMenu, SelectedPackagePaths, SelectedAssetPackages);
}

void FStormSyncAssetFolderContextMenu::PopulateAssetFileContextMenu(UToolMenu* InMenu)
{
	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::PopulateAssetFileContextMenu - InMenu: %ls", *GetNameSafe(InMenu));
	check(InMenu);

	const UContentBrowserDataMenuContext_FileMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_FileMenu was missing!"));

	// Extract the internal package paths that belong to this data source from the full list of selected items given in the context
	TArray<FName> SelectedAssetPackages;
	GetSelectedFiles(InMenu, SelectedAssetPackages);

	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::PopulateAssetFileContextMenu - Result ...");
	UE_LOGF(LogStormSyncEditor, Verbose, "\tSelectedAssetPackages: %d", SelectedAssetPackages.Num());
	for (const FName& SelectedPackageName : SelectedAssetPackages)
	{
		UE_LOGF(LogStormSyncEditor, Verbose, "\t- %ls", *SelectedPackageName.ToString());
	}

	AddFileMenuOptions(InMenu, SelectedAssetPackages);
}

void FStormSyncAssetFolderContextMenu::AddFileMenuOptions(UToolMenu* InMenu, const TArray<FName>& InSelectedPackageNames)
{
	const UContentBrowserDataMenuContext_FileMenu* Context = InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	checkf(Context, TEXT("Required context UContentBrowserDataMenuContext_FileMenu was missing!"));

	const bool bCanBeModified = !Context || Context->bCanBeModified;
	if (!bCanBeModified)
	{
		return;
	}

	// Figure out if selection is containing dirty (unsaved) assets
	FText DisabledTooltipReason;
	const TArray<FAssetData> DirtyAssets = GetDirtyAssets(InSelectedPackageNames, DisabledTooltipReason);
	
	bool bContainsDirtyAssets = !DirtyAssets.IsEmpty();
	const FCanExecuteAction CanExecuteDelegate = FCanExecuteAction::CreateLambda([bContainsDirtyAssets]() { return !bContainsDirtyAssets; });

	FToolMenuSection& Section = InMenu->AddSection(TEXT("StormSyncActions"), LOCTEXT("StormSyncActionsMenuHeading", "Storm Sync"));
	Section.InsertPosition = FToolMenuInsert(TEXT("AssetContextReferences"), EToolMenuInsertType::Before);

	// Sync Assets Action
	{
		Section.AddMenuEntry(
			TEXT("StormSync_SyncAssets"),
			LOCTEXT("SyncAssetsMenuEntry", "Sync Asset(s)"),
			FText::Format(LOCTEXT("SyncAssetsMenuEntryTooltip", "Broadcast a synchronisation for the selected assets on Storm Sync connected devices.{0}"), DisabledTooltipReason),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Server"),
			FUIAction(
				FExecuteAction::CreateStatic(&FStormSyncAssetFolderContextMenu::ExecuteSyncAssetsAction, InSelectedPackageNames),
				CanExecuteDelegate
			)
		);
	}

	constexpr bool bOpenSubMenuOnClick = false;

	// Push Assets Action
	{
		Section.AddSubMenu(
			TEXT("StormSync_PushAssets"),
			LOCTEXT("PushAssetsMenuEntry", "Push Asset(s)"),
			FText::Format(LOCTEXT("PushAssetsMenuEntryTooltip", "Push the selected assets to Storm Sync specific remote.{0}"), DisabledTooltipReason),
			FNewMenuDelegate::CreateRaw(
				this,
				&FStormSyncAssetFolderContextMenu::BuildPushAssetsMenuSection,
				InSelectedPackageNames,
				true
			),
			FUIAction(
				FExecuteAction(),
				CanExecuteDelegate
			),
			EUserInterfaceActionType::Button,
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll")
		);
	}

	// Pull Assets Action
	{
		FText StatusText;
		bool bIsServerRunning = IStormSyncTransportServerModule::Get().GetServerStatus(StatusText);

		const FText MenuLabel = bIsServerRunning ? 
			LOCTEXT("PullAssetsMenuEntry", "Pull Asset(s)") :
			FText::Format(LOCTEXT("PullAssetsMenuEntry_ServerDisabled", "Pull Asset(s) - {0}"), StatusText);

		TArray<FText> MenuTooltipTexts;
		MenuTooltipTexts.Add(LOCTEXT("PullAssetsMenuEntryTooltip", "Pull the selected assets from Storm Sync specific remote."));

		if (!bIsServerRunning)
		{
			MenuTooltipTexts.Add(LOCTEXT(
				"PullAssetsMenuEntry_ServerDisabledTooltip",
				"\n\nThis action is disabled because Storm Sync Server is not running.\nRun `StormSync.Server.Start` or configure the server with auto-start setting enabled."
			));
		}
		
		if (bContainsDirtyAssets)
		{
			MenuTooltipTexts.Add(DisabledTooltipReason);
		}

		Section.AddSubMenu(
			TEXT("StormSync_PullAssets"),
			MenuLabel,
			FText::Join(FText::GetEmpty(), MenuTooltipTexts),
			FNewMenuDelegate::CreateRaw(
				this,
				&FStormSyncAssetFolderContextMenu::BuildPushAssetsMenuSection,
				InSelectedPackageNames,
				false
			),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([bContainsDirtyAssets, bIsServerRunning]() { return !bContainsDirtyAssets && bIsServerRunning; })
			),
			EUserInterfaceActionType::Button,
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import")
		);
	}

	// Compare Assets Action
	{
		Section.AddSubMenu(
			TEXT("StormSync_CompareAssets"),
			LOCTEXT("CompareAssetsMenuEntry", "Compare Asset(s) with"),
			FText::Format(LOCTEXT("CompareAssetsMenuEntryTooltip", "Compare asset(s) with a specific remote and see if files (and inner dependencies) are either missing or in mismatched state.{0}"), DisabledTooltipReason),
			FNewMenuDelegate::CreateRaw(
				this,
				&FStormSyncAssetFolderContextMenu::BuildCompareWithMenuSection,
				InSelectedPackageNames
			),
			FUIAction(
				FExecuteAction(),
				CanExecuteDelegate
			),
			EUserInterfaceActionType::Button,
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.Visualizers")
		);
	}

	// Export Assets Action
	{
		Section.AddMenuEntry(
			TEXT("StormSync_ExportAssets"),
			LOCTEXT("FolderExportAsAssetsMenuEntry", "Export"),
			FText::Format(LOCTEXT("FolderExportAsAssetsMenuEntryTooltip", "Export assets (and all inner dependencies) locally.{0}"), DisabledTooltipReason),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
			FUIAction(
				FExecuteAction::CreateStatic(&FStormSyncAssetFolderContextMenu::ExecuteExportAction, InSelectedPackageNames),
				CanExecuteDelegate
			)
		);
	}
}

void FStormSyncAssetFolderContextMenu::AddFolderMenuOptions(UToolMenu* InMenu, const TArray<FString>& InSelectedPackagePaths, const TArray<FString>& InSelectedAssetPackages)
{
	const UContentBrowserDataMenuContext_FolderMenu* Context = InMenu->FindContext<UContentBrowserDataMenuContext_FolderMenu>();
	checkf(Context, TEXT("Required context UContentBrowserDataMenuContext_FolderMenu was missing!"));

	const bool bCanBeModified = !Context || Context->bCanBeModified;
	if (!bCanBeModified)
	{
		return;
	}

	// Discover all the files for these folders
	TArray<FName> SelectedPackageNames;
	
	// First, gather all the files from asset registry from the passed in list of selected package paths (eg. folders)
	GetSelectedAssetsInPaths(InSelectedPackagePaths, SelectedPackageNames);
	
	// Then, make sure to include any individual selected assets (eg. files)
	SelectedPackageNames.Append(InSelectedAssetPackages);

	FToolMenuSection& Section = InMenu->AddSection(TEXT("StormSyncActions"), LOCTEXT("StormSyncActionsMenuHeading", "Storm Sync"));
	Section.InsertPosition = FToolMenuInsert(TEXT("PathContextBulkOperations"), EToolMenuInsertType::Before);

	// Export Assets Action
	{
		Section.AddMenuEntry(
			TEXT("StormSync_ExportAssets"),
			LOCTEXT("FileExportAsAssetsMenuEntry", "Export"),
			LOCTEXT("FileExportAsAssetsMenuEntryTooltip", "Export assets (and all inner dependencies) locally"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
			FUIAction(
				FExecuteAction::CreateStatic(&FStormSyncAssetFolderContextMenu::ExecuteExportAction, SelectedPackageNames)
			)
		);
	}
}

void FStormSyncAssetFolderContextMenu::GetSelectedAssetsInPaths(const TArray<FString>& InPaths, TArray<FName>& OutSelectedPackageNames)
{
	const FString& SourcesPath = InPaths.IsValidIndex(0) ? InPaths[0] : TEXT("");
	if (ensure(SourcesPath.Len()))
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if (AssetRegistryModule.Get().IsLoadingAssets())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ExportFolderAssetsNotDiscovered", "You must wait until asset discovery is complete to migrate a folder"));
			return;
		}

		TArray<FAssetData> AssetDataList;
		AssetViewUtils::GetAssetsInPaths(InPaths, AssetDataList);

		for (const FAssetData& AssetData : AssetDataList)
		{
			// Don't collect the external packages here
			if (AssetData.GetOptionalOuterPathName().IsNone())
			{
				OutSelectedPackageNames.Add(AssetData.PackageName);
			}
		}
	}
}

void FStormSyncAssetFolderContextMenu::GetSelectedFilesAndFolders(const UToolMenu* InMenu, TArray<FString>& OutSelectedPackagePaths, TArray<FString>& OutSelectedAssetPackages) const
{
	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::GetSelectedFilesAndFolders - InMenu: %ls", *GetNameSafe(InMenu));
	check(InMenu);

	const UContentBrowserDataMenuContext_FolderMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_FolderMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_FolderMenu was missing!"));

	// Extract the internal package paths that belong to this data source from the full list of selected items given in the context
	for (const FContentBrowserItem& SelectedItem : ContextObject->SelectedItems)
	{
		const FContentBrowserItemData* SelectedItemData = SelectedItem.GetPrimaryInternalItem();
		if (!SelectedItemData)
		{
			continue;
		}

		UContentBrowserDataSource* const DataSource = SelectedItemData->GetOwnerDataSource();
		if (!DataSource)
		{
			continue;
		}

		for (const FContentBrowserItemData& Item : SelectedItem.GetInternalItems())
		{
			FName PackagePath;
			FAssetData AssetData;
			if (DataSource->Legacy_TryGetPackagePath(Item, PackagePath))
			{
				OutSelectedPackagePaths.Add(PackagePath.ToString());
			}
			else if (DataSource->Legacy_TryGetAssetData(Item, AssetData))
			{
				OutSelectedAssetPackages.Add(AssetData.PackageName.ToString());
			}
		}
	}
}

void FStormSyncAssetFolderContextMenu::GetSelectedFiles(const UToolMenu* InMenu, TArray<FString>& OutSelectedAssetPackages) const
{
	UE_LOGF(LogStormSyncEditor, Verbose, "FStormSyncAssetFolderContextMenu::GetSelectedFiles - InMenu: %ls", *GetNameSafe(InMenu));
	check(InMenu);

	const UContentBrowserDataMenuContext_FileMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_FileMenu was missing!"));

	// Extract the internal package paths that belong to this data source from the full list of selected items given in the context
	for (const FContentBrowserItem& SelectedItem : ContextObject->SelectedItems)
	{
		const FContentBrowserItemData* SelectedItemData = SelectedItem.GetPrimaryInternalItem();
		if (!SelectedItemData)
		{
			continue;
		}

		UContentBrowserDataSource* const DataSource = SelectedItemData->GetOwnerDataSource();
		if (!DataSource)
		{
			continue;
		}

		for (const FContentBrowserItemData& Item : SelectedItem.GetInternalItems())
		{
			FAssetData AssetData;
			if (DataSource->Legacy_TryGetAssetData(Item, AssetData))
			{
				OutSelectedAssetPackages.Add(AssetData.PackageName.ToString());
			}
		}
	}
}

void FStormSyncAssetFolderContextMenu::GetSelectedFiles(const UToolMenu* InMenu, TArray<FName>& OutSelectedAssetPackages) const
{
	TArray<FString> SelectedAssetPackages;
	GetSelectedFiles(InMenu, SelectedAssetPackages);

	// Transform selected asset data into their package name equivalent
	TArray<FName> SelectedPackageNames;
	Algo::Transform(SelectedAssetPackages, SelectedPackageNames, [](const FString& AssetData)
	{
		return FName(*AssetData);
	});

	OutSelectedAssetPackages = MoveTemp(SelectedPackageNames);
}

void FStormSyncAssetFolderContextMenu::BuildPushAssetsMenuEntries(FMenuBuilder& InMenuBuilder, TArray<FName> InPackageNames, const bool bInIsPushing) const
{
	TMap<FMessageAddress, FStormSyncConnectedDevice> RegisteredConnections = FStormSyncEditorModule::Get().GetRegisteredConnections();

	// Deal with entry if there is no valid registered connections currently
	if (RegisteredConnections.IsEmpty())
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("NoRemotesConnected", "No valid connections currently detected on the network."),
			LOCTEXT("NoRemotesConnectedTooltip", "No valid connections currently detected on the network."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateStatic([]() { return false; })
			)
		);

		return;
	}

	// Deal with entries for each active registered connections
	for (const TPair<FMessageAddress, FStormSyncConnectedDevice>& RegisteredConnection : RegisteredConnections)
	{
		const FMessageAddress& DiscoveryMessageAddress = RegisteredConnection.Key;
		const FStormSyncConnectedDevice Connection = RegisteredConnection.Value;

		const FString& ClientAddressId = Connection.StormSyncClientAddressId;
		
		FMessageAddress ClientAddress;
		if (!FMessageAddress::Parse(ClientAddressId, ClientAddress))
		{
			UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::BuildPushAssetsMenuEntries - Error parsing message address for storm sync client with id \"%ls\"", *ClientAddressId);
			continue;
		}

		const FText RemoteDescription = FText::FromString(FString::Printf(TEXT("%s (%s)"), *Connection.HostName, *Connection.ProjectDir));
		FText LabelText = FText::Format(
			LOCTEXT("PushPullAssetMenuAction", "{0} {1} selected asset(s) to {2}"),
			bInIsPushing ? LOCTEXT("PushAssetMenuAction", "Push") : LOCTEXT("PullAssetMenuAction", "Pull"),
			InPackageNames.Num(),
			RemoteDescription
		);

		// Adjust menu entry label for push if server is not running (if the menu entry is disabled)
		if (bInIsPushing && !Connection.bIsServerRunning)
		{
			LabelText = FText::Format(
				LOCTEXT("PushPullAssetMenuAction_ServerDisabled", "Unable to push {0} selected asset(s) - Server is not running on {1}"),
				InPackageNames.Num(),
				RemoteDescription
			);
		}
		
		const FText LabelTooltip = GetEntryTooltipForRemote(DiscoveryMessageAddress, Connection);

		FExecuteAction Delegate = bInIsPushing ?
			FExecuteAction::CreateStatic(&FStormSyncAssetFolderContextMenu::ExecutePushAssetsAction, InPackageNames, ClientAddress.ToString()) :
			FExecuteAction::CreateStatic(&FStormSyncAssetFolderContextMenu::ExecutePullAssetsAction, InPackageNames, ClientAddress.ToString());

		bool bIsValidConnection = Connection.State == EStormSyncConnectedDeviceState::State_Active;
		if (bInIsPushing && !Connection.bIsServerRunning)
		{
			// For push action, disable the entry if remote server is not running as it needs to accept incoming connection to be able to push assets
			bIsValidConnection = false;
		}

		InMenuBuilder.AddMenuEntry(
			LabelText,
			LabelTooltip,
			FSlateIcon(),
			FUIAction(
				MoveTemp(Delegate),
				FCanExecuteAction::CreateLambda([bIsValidConnection]() { return bIsValidConnection; })
			)
		);
	}
}

void FStormSyncAssetFolderContextMenu::ExecuteSyncAssetsAction(TArray<FName> InPackageNames)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecuteSyncAssetsAction - Package names %d", InPackageNames.Num());
	for (FName PackageName : InPackageNames)
	{
		UE_LOGF(LogStormSyncEditor, Display, "\t- Package name: %ls", *PackageName.ToString());
	}

	// Note: We sync with a dummy package descriptor, next iterations could add in there an additional UI step.
	// Something that could be handled with a bit more UI integration, like some kind of popup window or wizard.
	const FStormSyncPackageDescriptor PackageDescriptor;
	IStormSyncTransportClientModule::Get().SynchronizePackages(PackageDescriptor, InPackageNames);
}

void FStormSyncAssetFolderContextMenu::ExecutePushAssetsAction(TArray<FName> InPackageNames, FString InMessageAddressId)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecutePushAssetsAction to %ls - Package names %d", *InMessageAddressId, InPackageNames.Num());

	FMessageAddress RemoteMessageAddress;
	if (!FMessageAddress::Parse(InMessageAddressId, RemoteMessageAddress))
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::ExecutePushAssetsAction to %ls - Unable to parse into a Message Address", *InMessageAddressId);
		return;
	}

	// Each sync "transaction" (in this case pull) gets a new message log page, to categorize each request / response
	// This also avoids an issue with previous errors from altering the state of the notification icon, even in case of successful pull.
	// Using a new page will flush the logs
	// Note: Ideally, we'd like to have pull request / response message ID displayed here too, but we don't know it just yet
	UStormSyncNotificationSubsystem::Get().NewPage(LOCTEXT("Push_Page_Label", "Push Packages"));

	UStormSyncNotificationSubsystem::Get().Info(FText::Format(
		LOCTEXT("Start_PushAssets", "Pushing {0} package names to {1} ..."),
		FText::AsNumber(InPackageNames.Num()),

		// TODO: Display remote in a user friendly way
		FText::FromString(InMessageAddressId)
	));
	
	const FOnStormSyncPushComplete Delegate = FOnStormSyncPushComplete::CreateLambda([](const TSharedPtr<FStormSyncTransportPushResponse>& Response)
	{
		check(Response.IsValid())
		UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecutePushAssetsAction - Got a response: %ls", *Response->ToString());
		UStormSyncNotificationSubsystem::Get().HandlePushResponse(Response);
	});
	
	const FStormSyncPackageDescriptor PackageDescriptor;
	IStormSyncTransportClientModule::Get().PushPackages(PackageDescriptor, InPackageNames, RemoteMessageAddress, Delegate);
}

void FStormSyncAssetFolderContextMenu::ExecutePullAssetsAction(TArray<FName> InPackageNames, FString InMessageAddressId)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecutePullAssetsAction to %ls - Package names %d", *InMessageAddressId, InPackageNames.Num());

	FMessageAddress RemoteMessageAddress;
	if (!FMessageAddress::Parse(InMessageAddressId, RemoteMessageAddress))
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::ExecutePullAssetsAction to %ls - Unable to parse into a Message Address", *InMessageAddressId);
		return;
	}

	// Each sync "transaction" (in this case pull) gets a new message log page, to categorize each request / response
	// This also avoids an issue with previous errors from altering the state of the notification icon, even in case of successful pull.
	// Using a new page will flush the logs
	// Note: Ideally, we'd like to have pull request / response message ID displayed here too, but we don't know it just yet
	UStormSyncNotificationSubsystem::Get().NewPage(LOCTEXT("Pull_Page_Label", "Pull Packages"));

	UStormSyncNotificationSubsystem::Get().Info(FText::Format(
		LOCTEXT("Start_PullAssets", "Pulling {0} package names from {1} ..."),
		FText::AsNumber(InPackageNames.Num()),

		// TODO: Display remote in a user friendly way
		FText::FromString(InMessageAddressId)
	));
	
	const FOnStormSyncPullComplete Delegate = FOnStormSyncPullComplete::CreateLambda([](const TSharedPtr<FStormSyncTransportPullResponse>& Response)
	{
		check(Response.IsValid())
		UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecutePullAssetsAction - Got a response: %ls", *Response->ToString());
		UStormSyncNotificationSubsystem::Get().HandlePullResponse(Response);
	});
	
	const FStormSyncPackageDescriptor PackageDescriptor;
	IStormSyncTransportClientModule::Get().PullPackages(PackageDescriptor, InPackageNames, RemoteMessageAddress, Delegate);
}

void FStormSyncAssetFolderContextMenu::ExecuteCompareWithAction(const TArray<FName> InPackageNames, const FString InMessageAddressId)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecuteCompareWithAction to %ls - Package names %d", *InMessageAddressId, InPackageNames.Num());

	FMessageAddress RemoteAddress;
	if (!FMessageAddress::Parse(InMessageAddressId, RemoteAddress))
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::ExecuteCompareWithAction to %ls - Unable to parse into a Message Address", *InMessageAddressId);
		return;
	}

	const FOnStormSyncRequestStatusComplete DoneDelegate = FOnStormSyncRequestStatusComplete::CreateLambda([](const TSharedPtr<FStormSyncTransportStatusResponse>& Response)
	{
		UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecuteCompareWithAction - Received response! %ls", *Response->ToString());

		// Prompt the user displaying all assets that are going to be migrated
		SStormSyncStatusWidget::OpenDialog(Response.ToSharedRef());
	});
	
	IStormSyncTransportClientModule::Get().RequestPackagesStatus(RemoteAddress, InPackageNames, DoneDelegate);
}

void FStormSyncAssetFolderContextMenu::ExecuteExportAction(const TArray<FName> InPackageNames)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::ExecuteExportAction - Package names %d", InPackageNames.Num());

	// Form a full list of packages to move by including the dependencies of the supplied packages
	TArray<FName> AllPackageNamesToMove;
	{
		FScopedSlowTask SlowTask(InPackageNames.Num(), LOCTEXT("MigratePackages_GatheringDependencies", "Gathering Dependencies..."));
		SlowTask.MakeDialog();

		SlowTask.EnterProgressFrame();
		
		FText ErrorText;
		if (!FStormSyncCoreUtils::GetDependenciesForPackages(InPackageNames, AllPackageNamesToMove, ErrorText))
		{
			UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::ExecuteExportAction - Failed to gather dependencies: %ls", *ErrorText.ToString());
			WarnOnInvalidPackages(InPackageNames, LOCTEXT("Export_Warning_Notify_Heading", "Some files are invalid (not existing on disk)"));
			return;
		}
	}

	// Validate and filter out any invalid references
	TArray<FName> FilteredPackageNames;

	// If user opted to always ignore non existing file on disk
	if (GetDefault<UStormSyncCoreSettings>()->bFilterInvalidReferences)
	{
		// Just filter them out and ignore
		FilteredPackageNames = AllPackageNamesToMove.FilterByPredicate([](const FName& PackageName)
		{
			return FPackageName::DoesPackageExist(PackageName.ToString());
		});
	}
	else
	{
		FilteredPackageNames = AllPackageNamesToMove;
		
		// If user opted to not filter invalid references, still display them as warnings with an in-editor notification
		WarnOnInvalidPackages(AllPackageNamesToMove, LOCTEXT("Export_Warning_Notify_Heading", "Some files are invalid (not existing on disk)"));
	}
	
	// Confirm that there is at least one package to move 
	if (FilteredPackageNames.Num() == 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ExportAs_NoFilesToSave", "No files were found to export"));
		return;
	}

	// Prompt the user displaying all assets that are going to be migrated
	SStormSyncExportWizard::OpenWizard(
		InPackageNames,
		FilteredPackageNames,
		SStormSyncExportWizard::FOnExportWizardCompleted::CreateStatic(&FStormSyncAssetFolderContextMenu::OnExportWizardCompleted)
	);
}

void FStormSyncAssetFolderContextMenu::OnExportWizardCompleted(const TArray<FName>& InPackageNames, const FString& InFilepath)
{
	// Run validation again, preventing hitting the CreatePakBuffer error code path just below in case some packages cannot be found on disk
	const FText ErrorHeadingText = LOCTEXT("Export_Error_Notify_Heading", "Error during export, some files were invalid (not existing on disk)");
	if (!WarnOnInvalidPackages(InPackageNames, ErrorHeadingText, FAppStyle::GetBrush(TEXT("AssetEditor.CompileStatus.Overlay.Error"))))
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::OnExportWizardCompleted - Some files are invalid, preventing export.");
		// Note: Opting to prevent operation by design when Settings' bFilterInvalidReferences is false (but use it to display warnings),
		// this can arguably be changed to not prevent operations (pak buffer creation below) by filtering invalid refs no matter what (even if bFilterInvalidReferences
		// is false)
		return;
	}
	
	// Create buffer based on provided package names
	{
		FScopedSlowTask SlowTask(InPackageNames.Num(), LOCTEXT("ExportPackages_CreatingBuffer", "Creating Buffer..."));
		SlowTask.MakeDialog();

		TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*InFilepath));
		if (!FileWriter.IsValid())
		{
			UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::OnExportWizardCompleted - Error creating file writer");
			return;
		}
		
		const FStormSyncCoreUtils::FOnFileAdded Delegate = FStormSyncCoreUtils::FOnFileAdded::CreateLambda([&SlowTask](const FStormSyncFileDependency& FileDependency)
		{
			SlowTask.EnterProgressFrame();
		});
	
		FText ErrorText;
		TArray<FStormSyncFileDependency> PackedFiles;
		if (!FStormSyncCoreUtils::CreatePakBuffer(InPackageNames, *FileWriter, ErrorText, Delegate))
		{
			FileWriter.Reset();
			IFileManager::Get().Delete(*InFilepath);
			UE_LOGF(LogStormSyncEditor, Error, "FStormSyncAssetFolderContextMenu::OnExportWizardCompleted - CreatePakBuffer Error: %ls", *ErrorText.ToString());
			return;
		}
	}

	// Notify user
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncAssetFolderContextMenu::OnExportWizardCompleted - Created file %ls", *InFilepath);

	const FString DirectoryPath = FPaths::GetPath(InFilepath);
	const FString BaseFilename = FPaths::GetBaseFilename(InFilepath);
	const FString AbsolutePath = FPaths::ConvertRelativePathToFull(DirectoryPath) / TEXT(""); // Ensure trailing /

	const FText InfoText = FText::Format(LOCTEXT("Notification_SavedPak", "Created archive \"{0}.spak\"\nin {1}"), FText::FromString(BaseFilename), FText::FromString(AbsolutePath));
	FNotificationInfo Info = FNotificationInfo(InfoText);
	Info.ExpireDuration = 5.0f;
	Info.FadeInDuration = 0.2f;
	Info.FadeOutDuration = 1.0f;
	Info.bFireAndForget = false;
	Info.bAllowThrottleWhenFrameRateIsLow = false;

	Info.Hyperlink = FSimpleDelegate::CreateLambda([AbsolutePath]()
	{
		const FString HyperlinkTarget = TEXT("file://") / AbsolutePath;
		UE_LOGF(LogStormSyncEditor, 
			Verbose,
			"FStormSyncAssetFolderContextMenu::OnExportWizardCompleted - Clicked on \"Open Folder\" link with Directory: %ls (HyperlinkTarget: %ls)",
			*AbsolutePath,
			*HyperlinkTarget
		);

		FPlatformProcess::LaunchURL(*HyperlinkTarget, nullptr, nullptr);
	});

	Info.HyperlinkText = LOCTEXT("Notification_OpenFolder", "Open Folder");

	const TSharedPtr<SNotificationItem> Notification = UStormSyncNotificationSubsystem::Get().AddSimpleNotification(Info);
	Notification->ExpireAndFadeout();
}

bool FStormSyncAssetFolderContextMenu::WarnOnInvalidPackages(const TArray<FName>& InPackageNames, const FText& InHeadingText, const FSlateBrush* InBrush)
{
	TArray<FString> ErrorMessages;
	for (const FName& PackageName : InPackageNames)
	{
		if (!FPackageName::DoesPackageExist(PackageName.ToString()))
		{
			FText ErrorMessage = FText::Format(LOCTEXT("Export_InvalidFile", "{0} does not exist on disk"), FText::FromName(PackageName));
			ErrorMessages.Add(ErrorMessage.ToString());
		}
	}
	
	if (!ErrorMessages.IsEmpty())
	{
		const FString ErrorsFileList = TEXT("- ") + FString::Join(ErrorMessages, TEXT("\n- "));

		const FText NotificationText = FText::Format(
			LOCTEXT("Export_Error_Notify_Format", "{0}:\n\n{1}"),
			InHeadingText,
			FText::FromString(ErrorsFileList)
		);
			
		UE_LOGF(LogStormSyncEditor, Warning, "FStormSyncAssetFolderContextMenu::WarnOnInvalidPackages - %ls", *NotificationText.ToString());

		FNotificationInfo NotifyInfo(NotificationText);
		NotifyInfo.WidthOverride = 480.f;
		NotifyInfo.ExpireDuration = 5.0f;
		NotifyInfo.FadeInDuration = 0.2f;
		NotifyInfo.FadeOutDuration = 1.0f;
		NotifyInfo.Image = InBrush != nullptr ? InBrush : FAppStyle::GetBrush(TEXT("AssetEditor.CompileStatus.Overlay.Warning"));
		
		UStormSyncNotificationSubsystem::Get().AddSimpleNotification(NotifyInfo);
	}

	return ErrorMessages.IsEmpty();
}

FText FStormSyncAssetFolderContextMenu::GetEntryTooltipForRemote(const FMessageAddress& InRemoteAddress, const FStormSyncConnectedDevice& InConnection)
{
	return FText::FromString(FString::Printf(
		TEXT("State: %s\nAddress ID: %s\nServer State: %s\nStorm Sync Server Address ID: %s,\nStorm Sync Client Address ID: %s,\nHostname: %s\nInstance Type: %s\nProject Name: %s\nProject Directory: %s"),
		*UEnum::GetValueAsString(InConnection.State),
		*InRemoteAddress.ToString(),
		InConnection.bIsServerRunning ? TEXT("Running") : TEXT("Stopped"),
		*InConnection.StormSyncServerAddressId,
		*InConnection.StormSyncClientAddressId,
		*InConnection.HostName,
		*UEnum::GetValueAsString(InConnection.InstanceType),
		*InConnection.ProjectName,
		*InConnection.ProjectDir
	));
}

#undef LOCTEXT_NAMESPACE
