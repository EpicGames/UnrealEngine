// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FMenuBuilder;
class UToolMenu;
struct FAssetData;
struct FMessageAddress;
struct FSlateBrush;
struct FStormSyncConnectedDevice;

class FStormSyncAssetFolderContextMenu : public TSharedFromThis<FStormSyncAssetFolderContextMenu>
{
public:
	void Initialize();
	void Shutdown();
	
	/**
	 * Submenu extension delegate to build submenu for Push and Pull actions
	 *
	 * This will create a new section and forward entries building to `BuildPushAssetsMenuEntries`
	 *
	 * @param InMenuBuilder Instance of MenuBuilder provided by the top level menu extension
	 * @param InPackageNames List of package names built from the selected assets (FAssetData) menu extensions provides
	 * @param bInIsPushing Whether the delegate should build the submenu for Push (true) or Pull (false) action
	 */
	void BuildPushAssetsMenuSection(FMenuBuilder& InMenuBuilder, const TArray<FName> InPackageNames, const bool bInIsPushing);
	
	/**
	 * Submenu extension delegate to build submenu for compare actions
	 *
	 * @param MenuBuilder Instance of MenuBuilder provided by the top level menu extension
	 * @param InPackageNames List of package names built from the selected assets (FAssetData) menu extensions provides
	 */
	void BuildCompareWithMenuSection(FMenuBuilder& MenuBuilder, TArray<FName> InPackageNames) const;

	/**
	 * Static helper to check if a set of PackageNames contains any unsaved (dirty) assets
	 *
	 * @param InPackageNames List of package names to check for dirty state
	 * 
	 * @return List of FAssetData with assets in dirty states, empty list otherwise
	 */
	static TArray<FAssetData> GetDirtyAssets(const TArray<FName>& InPackageNames);
	
	/**
	 * Static helper to check if a set of PackageNames contains any unsaved (dirty) assets, additionally returning a disabled reason
	 * (useful for UI extensions and display tooltips) if there is any package names in dirty state.
	 *
	 * @param InPackageNames List of package names to check for dirty state
	 * @param OutDisabledReason FText with disabled reason and list of dirty package names displayed as a bullet list,
	 * separated by new lines.
	 * 
	 * @return List of FAssetData with assets in dirty states, empty list otherwise
	 */
	static TArray<FAssetData> GetDirtyAssets(const TArray<FName>& InPackageNames, FText& OutDisabledReason);

private:
	/** Delegate handler to delay execution of menu extension until slate is ready */
	void RegisterMenus();

	/** Delegate to extend the content browser asset context menu */
	void PopulateAssetFolderContextMenu(UToolMenu* InMenu) const;
	
	/** Delegate to extend the content browser asset context menu */
	void PopulateAssetFileContextMenu(UToolMenu* InMenu);

	/** Extends menu with storm sync actions (when only files are selected) */
	void AddFileMenuOptions(UToolMenu* InMenu, const TArray<FName>& InSelectedPackageNames);
	
	/** Extends menu with storm sync actions (when folders are selected) */
	static void AddFolderMenuOptions(UToolMenu* InMenu, const TArray<FString>& InSelectedPackagePaths, const TArray<FString>& InSelectedAssetPackages);

	/** Asks asset registry for all the discovered files in the passed in paths (eg. folders) */
	static void GetSelectedAssetsInPaths(const TArray<FString>& InPaths, TArray<FName>& OutSelectedPackageNames);

	/** Helper to return the list of selected assets and folders */
	void GetSelectedFilesAndFolders(const UToolMenu* InMenu, TArray<FString>& OutSelectedPackagePaths, TArray<FString>& OutSelectedAssetPackages) const;
	
	/** Helper to return the list of selected assets */
	void GetSelectedFiles(const UToolMenu* InMenu, TArray<FString>& OutSelectedAssetPackages) const;
	
	/** Helper to return the list of selected assets (as FNames) */
	void GetSelectedFiles(const UToolMenu* InMenu, TArray<FName>& OutSelectedAssetPackages) const;
	
	/**
	 * Context menu builder helper to build submenu entries for Push and Pull actions
	 *
	 * This will create a single menu entry if no registered connection is currently detected with a
	 * "No valid connections currently detected on the network." message.
	 *
	 * And create a menu entry for each registered connection if we have any. Both the UIAction delegate for the entry and their label
	 * depends on bInIsPushing value.
	 *
	 * The enabled / disabled state of each entry depends on their active state (see FStormSyncConnectedDevice::bIsValid). If a
	 * connection becomes unresponsive, the entry for it will be disabled until it reaches the MessageBusTimeBeforeRemovingInactiveSource
	 * value defined in StormSyncTransportSettings (in which case, the connection is cleaned up and removed from submenu) or becomes
	 * responsive again
	 *
	 * @param InMenuBuilder Instance of MenuBuilder provided by the top level menu extension
	 * @param InPackageNames List of package names built from the selected assets (FAssetData) menu extensions provides
	 * @param bInIsPushing Whether the delegate should build the submenu for Push (true) or Pull (false) action
	 */
	void BuildPushAssetsMenuEntries(FMenuBuilder& InMenuBuilder, TArray<FName> InPackageNames, const bool bInIsPushing) const;
	
	/** Synchronize a list of assets over storm sync connected devices */
	static void ExecuteSyncAssetsAction(TArray<FName> InPackageNames);
	
	/** Action handler for Push Assets action in context menu for a given connected device */
	static void ExecutePushAssetsAction(TArray<FName> InPackageNames, FString InMessageAddressId);
	
	/** Action handler for Pull Assets action in context menu for a given connected device */
	static void ExecutePullAssetsAction(TArray<FName> InPackageNames, FString InMessageAddressId);
	
	/** Action handler for Push Assets action in context menu for a given connected device */
	static void ExecuteCompareWithAction(const TArray<FName> InPackageNames, const FString InMessageAddressId);
	
	/** Export a list of assets locally as a storm sync archive file */
	static void ExecuteExportAction(TArray<FName> InPackageNames);
	/**
	 * Handler for wizard completion, passing down the list of package to include and filepath destination
	 * for the buffer to create.
	 */
	static void OnExportWizardCompleted(const TArray<FName>& InPackageNames, const FString& InFilepath);

	/**
	 * Helper validating a list of package names, checking if they exist on disk and display as an in-editor notification warning
	 *
	 * @param InPackageNames List of package names to validate and check on file system
	 * @param InHeadingText Message text displayed as a heading before the list of invalid files
	 * @param InBrush Optional slate brush override. If not passed in, will use a default warning icon
	 *
	 * @return true is all package names are valid (eg. exists on disk), false otherwise (meaning a warning notification has been displayed)
	 */
	static bool WarnOnInvalidPackages(const TArray<FName>& InPackageNames, const FText& InHeadingText, const FSlateBrush* InBrush = nullptr);

	/** Static helper to return a formatted text to display as tooltip */
	static FText GetEntryTooltipForRemote(const FMessageAddress& InRemoteAddress, const FStormSyncConnectedDevice& InConnection);
};
