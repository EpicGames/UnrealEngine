// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStormSyncIpAddressComboBox.h"

#include "IPAddress.h"
#include "SocketSubsystem.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SEditableComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"

void SStormSyncIpAddressComboBox::Construct(const FArguments& InArgs)
{
	OnIPAddressSelectedDelegate = InArgs._OnIPAddressSelected;
	LocalAdapterAddressSource = GetLocalNetworkInterfaceCardIPs();
	
	ChildSlot
	[
		SAssignNew(LocalAdapterAddressComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&LocalAdapterAddressSource)
		.OnGenerateWidget(this, &SStormSyncIpAddressComboBox::GenerateLocalAdapterAddressComboBoxEntry)
		.OnSelectionChanged(this, &SStormSyncIpAddressComboBox::OnIpAddressSelected)
		.Content()
		[
			SAssignNew(IPAddressEditableTextBlock, SEditableText)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(FText::FromString(InArgs._InitialValue))
			.OnTextCommitted(this, &SStormSyncIpAddressComboBox::OnIPAddressTextCommited)
		]
	];
}

void SStormSyncIpAddressComboBox::OnIpAddressSelected(const TSharedPtr<FString> InAddress, ESelectInfo::Type InType) const
{
	const TSharedPtr<FString> SelectedIPAddress = LocalAdapterAddressComboBox->GetSelectedItem();
	if (SelectedIPAddress.IsValid())
	{
		IPAddressEditableTextBlock->SetText(FText::FromString(*SelectedIPAddress));
		OnIPAddressSelectedDelegate.ExecuteIfBound(*InAddress.Get());
	}
}

void SStormSyncIpAddressComboBox::OnIPAddressTextCommited(const FText& Text, ETextCommit::Type)
{
	OnIPAddressSelectedDelegate.ExecuteIfBound(Text.ToString());
}

TSharedRef<SWidget> SStormSyncIpAddressComboBox::GenerateLocalAdapterAddressComboBoxEntry(const TSharedPtr<FString> InAddress) const
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InAddress))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

TArray<TSharedPtr<FString>> SStormSyncIpAddressComboBox::GetLocalNetworkInterfaceCardIPs()
{
	TArray<TSharedPtr<FString>> LocalNetworkInterfaceCardIPs;

#if PLATFORM_WINDOWS
	// Add the default route IP Address, only for windows
	const FString DefaultRouteLocalAdapterAddress = TEXT("0.0.0.0");
	LocalNetworkInterfaceCardIPs.Add(MakeShared<FString>(DefaultRouteLocalAdapterAddress));
#endif
		
	// Add the local host IP address
	const FString LocalHostIpAddress = TEXT("127.0.0.1");
	LocalNetworkInterfaceCardIPs.Add(MakeShared<FString>(LocalHostIpAddress));

	TArray<TSharedPtr<FInternetAddr>> Addresses;
	ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(Addresses);
	for (const TSharedPtr<FInternetAddr>& Address : Addresses)
	{
		// Add unique, so in case the OS call returns with the local host or default route IP, we don't add it twice
		LocalNetworkInterfaceCardIPs.AddUnique(MakeShared<FString>(Address->ToString(false)));
	}

	return LocalNetworkInterfaceCardIPs;
}
