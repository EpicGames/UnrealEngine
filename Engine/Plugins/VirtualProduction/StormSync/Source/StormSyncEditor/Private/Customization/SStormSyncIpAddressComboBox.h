// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableText.h"

class SEditableTextBox;

/** Slate widget to list the addresses associated with the adapters on the local computer */
class SStormSyncIpAddressComboBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnChooseIP, FString)

	SLATE_BEGIN_ARGS(SStormSyncIpAddressComboBox) {}

	SLATE_ARGUMENT(FString, InitialValue)
	SLATE_EVENT(FOnChooseIP, OnIPAddressSelected)
	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	/** Cached list of addresses associated with the adapters on the local computer */
	TArray<TSharedPtr<FString>> LocalAdapterAddressSource;

	/** Cached pointer to combo box of addresses associated with the adapters on the local computer */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> LocalAdapterAddressComboBox;

	/** Text box shown on top of the local adapter address address combo box */
	TSharedPtr<SEditableText> IPAddressEditableTextBlock;

	/** Delegate executed when a local IP address was selected */
	FOnChooseIP OnIPAddressSelectedDelegate;

	/** Handles changes in the local adapter address combo box */
	void OnIpAddressSelected(TSharedPtr<FString> InAddress, ESelectInfo::Type InType) const;

	/** Called when the IP Address was commited in the editable text block */
	void OnIPAddressTextCommited(const FText&, ETextCommit::Type);

	/** Generates an entry in the local adapter address combo box */
	TSharedRef<SWidget> GenerateLocalAdapterAddressComboBoxEntry(TSharedPtr<FString> InAddress) const;

	/** Gets the list of addresses associated with the adapters on the local computer */
	static TArray<TSharedPtr<FString>> GetLocalNetworkInterfaceCardIPs();
};
