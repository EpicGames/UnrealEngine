// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportSettingsDetailsCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "SStormSyncIpAddressComboBox.h"
#include "StormSyncTransportSettings.h"

TSharedRef<IDetailCustomization> FStormSyncTransportSettingsDetailsCustomization::MakeInstance()
{
	return MakeShared<FStormSyncTransportSettingsDetailsCustomization>();
}

void FStormSyncTransportSettingsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	const UStormSyncTransportSettings* Settings = GetMutableDefault<UStormSyncTransportSettings>();
	CustomizeStringAsIpDropDown(Settings->TcpServerAddress, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStormSyncTransportSettings, TcpServerAddress)), DetailBuilder);
}

void FStormSyncTransportSettingsDetailsCustomization::CustomizeStringAsIpDropDown(const FString& InitialValue, TSharedPtr<IPropertyHandle> PropertyHandle, IDetailLayoutBuilder& DetailBuilder)
{
	IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(PropertyHandle);

	TSharedPtr<SWidget> NameWidget, ValueWidget;
	Row->GetDefaultWidgets(NameWidget, ValueWidget);
	
	Row->CustomWidget()
		.NameContent()
		[
			NameWidget->AsShared()
		]
		.ValueContent()
		[
			SNew(SStormSyncIpAddressComboBox)
			.InitialValue(InitialValue)
			.OnIPAddressSelected_Lambda([PropertyHandle](const FString& SelectedIp)
			{
				PropertyHandle->SetPerObjectValue(0, SelectedIp);
			})
		];
}
