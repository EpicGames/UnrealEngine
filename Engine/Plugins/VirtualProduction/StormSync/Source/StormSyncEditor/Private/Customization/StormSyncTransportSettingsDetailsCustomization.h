// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IPropertyHandle;

/** Details customization for Storm Sync Transport Settings */
class FStormSyncTransportSettingsDetailsCustomization : public IDetailCustomization
{
public:
	
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface

private:
	/** Replace standard FString with an ip dropdown combo box */
	void CustomizeStringAsIpDropDown(const FString& InitialValue, TSharedPtr<IPropertyHandle> PropertyHandle, IDetailLayoutBuilder& DetailBuilder);
};
