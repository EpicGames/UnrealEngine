// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncArchiveFactory.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncEditorLog.h"
#include "UObject/Package.h"

UStormSyncArchiveFactory::UStormSyncArchiveFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UStormSyncArchiveData::StaticClass();
	Formats.Add(TEXT("spak;Storm Sync Paked Archive"));

	bCreateNew = false;
	bText = false;
	bEditorImport = true;
	bEditAfterNew = false;
	
	FStormSyncCoreDelegates::OnFileImported.AddUObject(this, &UStormSyncArchiveFactory::OnFileImported);
}

UStormSyncArchiveFactory::~UStormSyncArchiveFactory()
{
	FStormSyncCoreDelegates::OnFileImported.RemoveAll(this);
}


bool UStormSyncArchiveFactory::FactoryCanImport(const FString& Filename)
{
	// TODO: Consider inspecting file to see if it is a valid buffer as serialized by storm sync
	return FPaths::GetExtension(Filename).Equals(TEXT("spak"));
}

UObject* UStormSyncArchiveFactory::ImportObject(UClass* InClass, UObject* InOuter, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Params, bool& OutCanceled)
{
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncArchiveFactory::ImportObject InClass: %ls, InOuter: %ls, InName: %ls, Filename: %ls", *GetNameSafe(InClass), *GetNameSafe(InOuter), *InName.ToString(), *Filename);

	// Import Object override to customize how import is done and account for this factory operating on a dummy imported object.

	UObject* Result = nullptr;
	AdditionalImportedObjects.Empty();
	CurrentFilename = Filename;

	// sanity check the file size of the impending import and prompt
	// the user if they wish to continue if the file size is very large
	const int64 FileSize = IFileManager::Get().FileSize(*CurrentFilename);

	if (!Filename.IsEmpty())
	{
		if (FileSize == INDEX_NONE)
		{
			UE_LOGF(LogStormSyncEditor, Error, "Can't find file '%ls' for import", *Filename);
		}
		else
		{
			UE_LOGF(LogStormSyncEditor, Display, "FactoryCreateFile: %ls with %ls (%i %i %ls)", *InClass->GetName(), *GetClass()->GetName(), bCreateNew, bText, *Filename);
			Result = FactoryCreateFile(InClass, InOuter, InName, Flags, *Filename, Params, GWarn, OutCanceled);
		}
	}

	// Deliberately not making package dirty and broadcasting post edit change,
	// since this factory operates on a dummy imported object and delegates the work of
	// creating / importing assets to the storm sync import editor subsystem

	if (Result != nullptr)
	{
		// Make sure the package is not dirty. It is going to be deleted.
		if (UPackage* Package = Result->GetOutermost())
		{
			Package->SetDirtyFlag(false);	
		}
	}

	CurrentFilename = TEXT("");

	return Result;
}

UObject* UStormSyncArchiveFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Params, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncArchiveFactory::FactoryCreateFile InClass: %ls, InOuter: %ls, InName: %ls, Filename: %ls", *GetNameSafe(InClass), *GetNameSafe(InParent), *InName.ToString(), *Filename);

	// We create the dummy imported object at the dropped location, otherwise UAssetToolsImpl::SyncBrowserToAssets will
	// select a fallback root folder (i.e. Engine/Content) which is undesired.
	// The dummy object will be deleted at the end of the import.
	// Note: adding RF_Transient avoids the package being marked dirty.
	UStormSyncArchiveData* ArchiveData = NewObject<UStormSyncArchiveData>(InParent, InClass, InName, Flags | RF_Transient);
	check(ArchiveData);
	ArchiveData->Filename = Filename;

	PendingImports.Add(Filename, FSoftObjectPath(ArchiveData));

	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncArchiveFactory::FactoryCreateFile for %ls", *GetNameSafe(ArchiveData));
	
	// Broadcast import event and delay import until next tick to avoid blocking the process that files were dragged from
	// (delay handled internally by Import subsystem)
	FStormSyncCoreDelegates::OnRequestImportFile.Broadcast(Filename);

	return ArchiveData;
}

bool UStormSyncArchiveFactory::CanReimport(UObject* InObject, TArray<FString>& OutFilenames)
{
	if (const UStormSyncArchiveData* ArchiveData = Cast<UStormSyncArchiveData>(InObject))
	{
		OutFilenames.Add(ArchiveData->Filename);
		return true;
	}
	return false;
}

void UStormSyncArchiveFactory::SetReimportPaths(UObject* InObject, const TArray<FString>& InNewReimportPaths)
{
	UStormSyncArchiveData* ArchiveData = Cast<UStormSyncArchiveData>(InObject);
	if (ArchiveData && ensure(InNewReimportPaths.Num() == 1))
	{
		ArchiveData->Filename = InNewReimportPaths[0];
	}
}

EReimportResult::Type UStormSyncArchiveFactory::Reimport(UObject* InObject)
{
	if (UStormSyncArchiveData* ArchiveData = Cast<UStormSyncArchiveData>(InObject))
	{
		PendingImports.Add(ArchiveData->Filename, FSoftObjectPath(ArchiveData));

		// Broadcast import event and delay import until next tick to avoid blocking the process that files were dragged from
		// (delay handled internally by Import subsystem)
		FStormSyncCoreDelegates::OnRequestImportFile.Broadcast(ArchiveData->Filename);
		return EReimportResult::Succeeded;
	}
	return EReimportResult::Failed;
}

void UStormSyncArchiveFactory::OnFileImported(const FString& InFilename)
{
	if (const FSoftObjectPath* FoundAssetPath = PendingImports.Find(InFilename))
	{
		if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
		{
			// Using DeleteAssets will properly notify everything.
			const FAssetData AssetData = AssetRegistry->GetAssetByObjectPath(*FoundAssetPath);
			if (AssetData.IsValid())
			{
				ObjectTools::DeleteAssets({}, false);
			}
		}

		// Rename the object (if it is still around) to avoid blocking reimport.
		if (UObject* AssetObject = FoundAssetPath->ResolveObject())
		{
			// Since deletion can be delayed, rename to avoid future name collision
			AssetObject->Rename( nullptr, GetTransientPackage(), REN_DontCreateRedirectors);

			// Mark the object for GC (eventually).
			AssetObject->RemoveFromRoot();
			AssetObject->ClearFlags(RF_Public | RF_Standalone);
			AssetObject->MarkAsGarbage();
		}

		PendingImports.Remove(InFilename);
	}
}