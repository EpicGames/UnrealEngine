// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorReimportHandler.h"
#include "Factories/Factory.h"
#include "StormSyncArchiveFactory.generated.h"

/** Dummy UObject created during import as transient packages */
UCLASS()
class UStormSyncArchiveData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Transient)
	FString Filename;
};

/**
 * Special Storm Sync Factory to handle import of storm sync archive files and hook into content browser
 * (drag & drop and import from context menu)
 *
 * It differs from usual factories as it operates on a dummy UObject transient package, and forwards handling
 * of package extraction and asset import / update to storm sync import subsystem.
 *
 * When exporting / importing via Storm Sync, we don't operate on an atomic .uasset representing the underlying
 * file it is importing, but more like a "zip" including all assets and their inner dependencies.
 */
UCLASS(hidecategories=Object, MinimalAPI)
class UStormSyncArchiveFactory : public UFactory, public FReimportHandler
{
	GENERATED_BODY()
public:
	explicit UStormSyncArchiveFactory(const FObjectInitializer& ObjectInitializer);
	~UStormSyncArchiveFactory();

	//~ Begin UFactory interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* ImportObject(UClass* InClass, UObject* InOuter, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Params, bool& OutCanceled) override;
	virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Params, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	//~ End UFactory interface

	//~ Begin FReimportHandler
	virtual bool CanReimport(UObject* InObject, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* InObject, const TArray<FString>& InNewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* InObject) override;
	//~ End FReimportHandler

	void OnFileImported(const FString& InFilename);

	TMap<FString, FSoftObjectPath> PendingImports;
};
