// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStormSyncExportWizard.h"

#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Misc/App.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Slate/SStormSyncFileDependencyWidgetRow.h"
#include "Slate/SStormSyncReportDialog.h"
#include "SlateOptMacros.h"
#include "StormSyncCoreSettings.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncEditorLog.h"
#include "StormSyncImportTypes.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Workflow/SWizard.h"

#define LOCTEXT_NAMESPACE "StormSyncExportWizard"

namespace StormSync::ExportWizardInternal
{
	static FString Default_FileExtension = TEXT(".spak");
}

void SStormSyncExportWizard::Construct(const FArguments& InArgs, const TArray<FName>& InInitialPackageNames, const TArray<FName>& InPackageNames, const FOnExportWizardCompleted& InOnExportWizardCompleted)
{
	OnWizardCompleted = InOnExportWizardCompleted;

	InitialPackageNames = InInitialPackageNames;
	SelectedPackageNames = InPackageNames;
	
	// Starts with empty filename
	ExportFilename = FText::FromString(GetDefaultNameFromSelectedPackages(InitialPackageNames));
	ExportDirectory = FText::FromString(FPaths::ProjectSavedDir() / TEXT("StormSync/"));
	
	bLastInputValidityCheckSuccessful = true;
	
	const FText ReportMessage = LOCTEXT("ExportAs_ReportTitle", "The following assets will be saved to a local pak file.");

	ReportPackages = MakeShareable(new TArray<FStormSyncReportPackageData>);

	for (const FName& PackageName : InPackageNames)
	{
		ReportPackages.Get()->Add(FStormSyncReportPackageData(PackageName.ToString(), true));
	}

	UpdateInputValidity();
	
	ChildSlot
	[
		SNew(SBorder)
		.Padding(18)
		.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
		[
			SAssignNew(Wizard, SWizard)
			.OnCanceled(this, &SStormSyncExportWizard::OnCancelButtonClicked)
			.OnFinished(this, &SStormSyncExportWizard::OnFinish)
			.CanFinish(this, &SStormSyncExportWizard::CanFinish)
			.ShowPageList(false)

			// Report Dialog page - User choose files to include here
			+SWizard::Page()
			.CanShow(true)
			[

				SAssignNew(ReportDialog, SStormSyncReportDialog, ReportMessage, *ReportPackages.Get())
			]

			// Choose destination page - User choose file destination here (and see the full list of included files)
			+SWizard::Page()
			.CanShow(this, &SStormSyncExportWizard::CanShowDestinationPage)
			.OnEnter(this, &SStormSyncExportWizard::OnDestinationPageEntered)
			[
				SNew(SVerticalBox)
				
				// Title
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("HeadingExtraSmall"))
					.Text(LOCTEXT("Export_Title_02", "Choose File Path destination"))
					.TransformPolicy(ETextTransformPolicy::ToUpper)
				]

				// Page Description
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 8.f, 0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DestinationPageDescription_01", "Enter a name for your pak buffer. Pak buffer names may only contain alphanumeric characters, and may not contain a space."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DestinationPageDescription_02", "When you click the \"Finish\" button below, a pak buffer (.spak) file will be created using this name."))
				]

				+SVerticalBox::Slot()
				.Padding(0.f, 16.f, 0.f, 0.f)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.Text(LOCTEXT("Export_FileName_Label", "File Name"))
				]

				+SVerticalBox::Slot()
				.Padding(0.f, 8.f, 0.f, 0.f)
				.AutoHeight()
				[
					SAssignNew(FilenameEditBox, SEditableTextBox)
					.Text(this, &SStormSyncExportWizard::GetFileNameText)
					.OnTextChanged(this, &SStormSyncExportWizard::OnFileNameTextChanged) 
			 	]

				// File Destination
				+SVerticalBox::Slot()
				.Padding(0.f, 16.f, 0.f, 0.f)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.Text(LOCTEXT("Export_Filepath_Label", "File Path"))
				]
				+SVerticalBox::Slot()
				.Padding(0.f, 8.f, 0.f, 0.f)
				.AutoHeight()
				[
					SNew(SBox)
					.HeightOverride(26.f)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SNew(SEditableTextBox)
							.IsEnabled(false)
							.Text(this, &SStormSyncExportWizard::GetFilePathText)
							.OnTextChanged(this, &SStormSyncExportWizard::OnFilePathTextChanged)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(6.0f, 1.0f, 0.0f, 0.0f)
						[
							SNew(SButton)
							.VAlign(VAlign_Center)
							.ButtonStyle(FAppStyle::Get(), "SimpleButton")
							.OnClicked(this, &SStormSyncExportWizard::HandleChooseFolderButtonClicked)
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.FolderClosed"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
					]
				]

				// File List
				+SVerticalBox::Slot()
				.Padding(0.f, 16.f, 0.f, 0.f)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.Text(LOCTEXT("Export_ListOfFiles_Label", "List of files to include"))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 8.f, 0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Export_ListOfFiles_Description", "Here is the list of files to be included in the pak buffer. You can go back to the previous step with the \"Back\" button below to edit this list."))
				]

				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.Padding(0.f, 8.f)
				[
					SAssignNew(FileDependenciesListView, SListView<TSharedPtr<FStormSyncFileDependency>>)
					.ListItemsSource(&FileDependencyList)
					.HeaderRow(SNew(SHeaderRow)
						+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_PackageName)
						.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_PackageName)
						.SortMode(this, &SStormSyncExportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_PackageName)
						.OnSort(this, &SStormSyncExportWizard::OnSortAttributeEntries)
						.FillWidth(0.4f)
						+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_FileSize)
						.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_FileSize)
						.SortMode(this, &SStormSyncExportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_FileSize)
						.OnSort(this, &SStormSyncExportWizard::OnSortAttributeEntries)
						.FillWidth(0.1f)
						+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_Timestamp)
						.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_Timestamp)
						.SortMode(this, &SStormSyncExportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_Timestamp)
						.OnSort(this, &SStormSyncExportWizard::OnSortAttributeEntries)
						.FillWidth(0.2f)
						+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_FileHash)
						.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_FileHash)
						.SortMode(this, &SStormSyncExportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_FileHash)
						.OnSort(this, &SStormSyncExportWizard::OnSortAttributeEntries)
						.FillWidth(0.2f)
					)
					.OnGenerateRow(this, &SStormSyncExportWizard::MakeFileDependencyWidget)
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(this, &SStormSyncExportWizard::GetExpectedFileSize)
				]
				
				// Error validation
				+SVerticalBox::Slot()
				.Padding(0.f, 16.f)
				.AutoHeight()
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Visibility(this, &SStormSyncExportWizard::GetNameErrorLabelVisibility)
					.Message(this, &SStormSyncExportWizard::GetNameErrorLabelText)
				]
			]
		]
	];
}

void SStormSyncExportWizard::OpenWizard(const TArray<FName>& InInitialPackageNames, const TArray<FName>& PackageNames, const FOnExportWizardCompleted& InOnExportWizardCompleted)
{
	const TSharedRef<SWindow> ReportWindow = SNew(SWindow)
		.Title(LOCTEXT("Window_Title", "Storm Sync | Export files to local pak"))
		.ClientSize(FVector2D(960, 700))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SStormSyncExportWizard, InInitialPackageNames, PackageNames, InOnExportWizardCompleted)
		];
	

	const IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	const TSharedPtr<SWindow> ParentWindow = MainFrameModule.GetParentWindow();
	
	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(ReportWindow, MainFrameModule.GetParentWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(ReportWindow);
	}
}

void SStormSyncExportWizard::CloseDialog()
{
	const TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

FString SStormSyncExportWizard::GetDefaultNameFromSelectedPackages(const TArray<FName>& InPackageNames)
{
	if (InPackageNames.IsEmpty())
	{
		return TEXT("");
	}

	FString PackageShortName;
	if (InPackageNames.Num() == 1 && InPackageNames.IsValidIndex(0))
	{
		PackageShortName = FPackageName::GetShortName(InPackageNames[0]);
	}
	else
	{
		// More than one, default to project name
		PackageShortName = FApp::GetProjectName();
	}

	const UStormSyncCoreSettings* Settings = GetDefault<UStormSyncCoreSettings>();
	check(Settings);

	if (Settings->ExportDefaultNameFormatString.IsEmpty())
	{
		return PackageShortName;
	}

	const FDateTime Now = FDateTime::Now();
	return FString::Printf(TEXT("%s_%s"), *PackageShortName, *Now.ToString(*Settings->ExportDefaultNameFormatString));
}

void SStormSyncExportWizard::OnDestinationPageEntered()
{
	FileDependencyList.Empty();

	const TArray<FName> PackageNamesToExport = GetPackageNamesFromReportsData();
	if (PackageNamesToExport.IsEmpty())
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncEditorModule::OnDestinationPageEntered - Can't proceed with empty list of files to include");
		if (FileDependenciesListView.IsValid())
		{
			FileDependenciesListView->RebuildList();
		}

		UpdateInputValidity();
		return;
	}
	
	FScopedSlowTask SlowTask(PackageNamesToExport.Num(), LOCTEXT("MigratePackages_GatheringDependencies", "Gathering Dependencies..."));
	SlowTask.MakeDialog();
	SlowTask.EnterProgressFrame();

	TArray<FStormSyncFileDependency> FileDependencies = FStormSyncCoreUtils::GetAvaFileDependenciesFromPackageNames(PackageNamesToExport);
	for (const FStormSyncFileDependency& FileDependency : FileDependencies)
	{
		UE_LOGF(LogStormSyncEditor, Display, "\tFileDependency: %ls", *FileDependency.ToString());
		FileDependencyList.Add(MakeShared<FStormSyncFileDependency>(FileDependency));
	}
	
	if (FileDependenciesListView.IsValid())
	{
		FileDependenciesListView->RebuildList();
	}
	
	UpdateInputValidity();
	
	TWeakPtr<SStormSyncExportWizard> LocalWeakThis = SharedThis(this);
	GEditor->GetTimerManager()->SetTimerForNextTick([LocalWeakThis]()
	{
		if (const TSharedPtr<SStormSyncExportWizard> StrongThis = LocalWeakThis.Pin(); StrongThis.IsValid())
		{
			// Set focus to the filename box on creation
			FSlateApplication::Get().SetKeyboardFocus(StrongThis->FilenameEditBox);
			FSlateApplication::Get().SetUserFocus(0, StrongThis->FilenameEditBox);
			StrongThis->FilenameEditBox->SelectAllText();
		}
	});
}

TArray<FName> SStormSyncExportWizard::GetPackageNamesFromReportsData() const
{
	TArray<FName> PackageNames;
	const TArray<FStormSyncReportPackageData>* ReportsDataPtr = ReportPackages.Get();
	if (!ReportsDataPtr)
	{
		UE_LOGF(LogStormSyncEditor, Error, "SStormSyncExportWizard::GetPackageNamesFromReportsData - Unable to get report data from dialog");
		return PackageNames;
	}

	TArray<FStormSyncReportPackageData> ReportsData = *ReportsDataPtr;
	for (const FStormSyncReportPackageData& ReportData : ReportsData)
	{
		if (ReportData.bShouldIncludePackage)
		{
			PackageNames.AddUnique(FName(*ReportData.Name));
		}
	}

	return PackageNames;
}

FText SStormSyncExportWizard::GetExpectedFileSize() const
{
	uint64 TotalFileSize = 0;
	for (const TSharedPtr<FStormSyncFileDependency>& FileDependency : FileDependencyList)
	{
		TotalFileSize += FileDependency->FileSize;
	}

	const FString ExpectedFileSize = FStormSyncCoreUtils::GetHumanReadableByteSize(TotalFileSize);
	return FText::Format(LOCTEXT("ExpectedFileSize", "File buffer with {0} assets will have an expected size of {1}"), FText::AsNumber(FileDependencyList.Num()), FText::FromString(ExpectedFileSize));
}

bool SStormSyncExportWizard::CanShowDestinationPage() const
{
	return !GetPackageNamesFromReportsData().IsEmpty();
}

TSharedRef<ITableRow> SStormSyncExportWizard::MakeFileDependencyWidget(const TSharedPtr<FStormSyncFileDependency> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	FStormSyncImportFileInfo FileInfo;
	FileInfo.FileDependency = *InItem.Get();
	return SNew(SStormSyncFileDependencyWidgetRow, OwnerTable).Item(MakeShared<FStormSyncImportFileInfo>(FileInfo));
}

void SStormSyncExportWizard::OnCancelButtonClicked()
{
	CloseDialog();
}

void SStormSyncExportWizard::OnFinish()
{
	check(CanFinish());
	
	CloseDialog();

	const TArray<FName> PackageNames = GetPackageNamesFromReportsData();
	const FString Filepath = ExportDirectory.ToString() / ExportFilename.ToString() + StormSync::ExportWizardInternal::Default_FileExtension;

	OnWizardCompleted.ExecuteIfBound(PackageNames, Filepath);
}

bool SStormSyncExportWizard::CanFinish() const
{
	return bLastInputValidityCheckSuccessful;
}

FText SStormSyncExportWizard::GetFileNameText() const
{
	return ExportFilename;
}

void SStormSyncExportWizard::OnFileNameTextChanged(const FText& InNewText)
{
	ExportFilename = InNewText;
	UpdateInputValidity();
}

FText SStormSyncExportWizard::GetFilePathText() const
{
	return ExportDirectory;
}

void SStormSyncExportWizard::OnFilePathTextChanged(const FText& InNewText)
{
	ExportDirectory = InNewText;
	UpdateInputValidity();
}

FReply SStormSyncExportWizard::HandleChooseFolderButtonClicked()
{
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		const void* ParentWindowWindowHandle = (ParentWindow.IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderName;
		const FString Title = LOCTEXT("ExportWizardBrowseTitle", "Choose a file location").ToString();

		if (DesktopPlatform->OpenDirectoryDialog(ParentWindowWindowHandle, Title, ExportDirectory.ToString(), FolderName))
		{
			if (!FolderName.EndsWith(TEXT("/")))
			{
				FolderName += TEXT("/");
			}

			ExportDirectory = FText::FromString(FolderName);
			
			UpdateInputValidity();
		}
	}

	return FReply::Handled();
}

EVisibility SStormSyncExportWizard::GetNameErrorLabelVisibility() const
{
	return bLastInputValidityCheckSuccessful ? EVisibility::Hidden : EVisibility::Visible;
}

FText SStormSyncExportWizard::GetNameErrorLabelText() const
{
	if (!bLastInputValidityCheckSuccessful)
	{
		return LastInputValidityErrorText;
	}

	return FText::GetEmpty();
}

void SStormSyncExportWizard::UpdateInputValidity()
{
	// Check validity for ExportFilename
	bLastInputValidityCheckSuccessful = IsValidFilenameForCreation(ExportFilename.ToString(), LastInputValidityErrorText);

	// Check Export directory too ? or rely on the fact that it can only be changed via OS file dialog and should
	// always a valid location (should either be the Saved project folder initially, or changed via file dialog)

	// Check for file list validity, must have at least one file to export
	if (bLastInputValidityCheckSuccessful && FileDependencyList.IsEmpty())
	{
		bLastInputValidityCheckSuccessful = false;
		LastInputValidityErrorText = LOCTEXT("Export_FileList_Empty", "At least one file must be selected to include in a pak buffer");
	}
}

bool SStormSyncExportWizard::IsValidFilenameForCreation(const FString& InFilename, FText& OutFailReason)
{
	const FString BaseFilename = FPaths::GetBaseFilename(InFilename);

	if (BaseFilename.IsEmpty())
	{
		OutFailReason = LOCTEXT("NoFileName", "You must specify a file name.");
		return false;
	}

	if (BaseFilename.Contains(TEXT(" ")))
	{
		OutFailReason = LOCTEXT("FileNameContainsSpace", "File names may not contain a space.");
		return false;
	}

	if (!FChar::IsAlpha(BaseFilename[0]))
	{
		OutFailReason = LOCTEXT("FileNameMustBeginWithACharacter", "File names must begin with an alphabetic character.");
		return false;
	}

	FString IllegalNameCharacters;
	if (!NameContainsOnlyLegalCharacters(BaseFilename, IllegalNameCharacters))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("IllegalNameCharacters"), FText::FromString(IllegalNameCharacters));
		OutFailReason = FText::Format(LOCTEXT("FileNameContainsIllegalCharacters", "File names may not contain the following characters: {IllegalNameCharacters}"), Args);
		return false;
	}
	
	return true;
}

bool SStormSyncExportWizard::NameContainsOnlyLegalCharacters(const FString& TestName, FString& OutIllegalCharacters)
{
	bool bContainsIllegalCharacters = false;

	// Only allow alphanumeric characters in the file name
	for (int32 CharIdx = 0; CharIdx < TestName.Len(); ++CharIdx)
	{
		const FString& Char = TestName.Mid(CharIdx, 1);
		if (!FChar::IsAlnum(Char[0]) && Char != TEXT("_"))
		{
			if (!OutIllegalCharacters.Contains(Char))
			{
				OutIllegalCharacters += Char;
			}

			bContainsIllegalCharacters = true;
		}
	}

	return !bContainsIllegalCharacters;
}

EColumnSortMode::Type SStormSyncExportWizard::GetSortModeForColumn(FName InColumnId) const
{
	if (ColumnIdToSort == InColumnId)
	{
		return ActiveSortMode;
	}
		
	return EColumnSortMode::None;
}

void SStormSyncExportWizard::OnSortAttributeEntries(EColumnSortPriority::Type InPriority, const FName& InColumnId, EColumnSortMode::Type InSortMode)
{
	
	ColumnIdToSort = InColumnId;
	ActiveSortMode = InSortMode;
	
	FileDependencyList.Sort([InColumnId, InSortMode](const TSharedPtr<FStormSyncFileDependency>& Left, const TSharedPtr<FStormSyncFileDependency>& Right)
	{		
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_FileSize)
		{
			return InSortMode == EColumnSortMode::Ascending ?
				Left->FileSize < Right->FileSize :
				Left->FileSize > Right->FileSize;
		}
		
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_Timestamp)
		{
			return InSortMode == EColumnSortMode::Ascending ?
				Left->Timestamp < Right->Timestamp :
				Left->Timestamp > Right->Timestamp;
		}
		
		int32 CompareResult = 0;
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_PackageName)
		{
			CompareResult = Left->PackageName.Compare(Right->PackageName);
		}

		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_FileHash)
		{
			CompareResult = Left->FileHash.Compare(Right->FileHash);
		}

		return InSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
	});

	FileDependenciesListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
