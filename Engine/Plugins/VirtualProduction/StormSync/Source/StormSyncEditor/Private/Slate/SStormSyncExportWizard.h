// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/SCompoundWidget.h"

struct FStormSyncFileDependency;
class SStormSyncReportDialog;
class SWizard;
class SEditableTextBox;

struct FStormSyncReportPackageData;

/** A modal dialog to collect information needed to export a pak buffer locally */
class SStormSyncExportWizard : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOnExportWizardCompleted, const TArray<FName>&, const FString&)

	SLATE_BEGIN_ARGS(SStormSyncExportWizard) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TArray<FName>& InInitialPackageNames, const TArray<FName>& InPackageNames, const FOnExportWizardCompleted& InOnExportWizardCompleted);

	/** Opens the dialog in a new window */
	static void OpenWizard(const TArray<FName>& InInitialPackageNames, const TArray<FName>& PackageNames, const FOnExportWizardCompleted& InOnExportWizardCompleted);

	/** Closes the dialog */
	void CloseDialog();
	
	/**
	 * Tries to find a default suitable name for the pak name export (based on user selection in content browser)
	 *
	 * Will either use the package name if only one asset is selected, or use project name if more than one asset
	 * is currently selected in content browser.
	 *
	 * Appends the current date time to the default name.
	 */
	static FString GetDefaultNameFromSelectedPackages(const TArray<FName>& InPackageNames);

private:
	/** Delegate triggered on wizard completion */
	FOnExportWizardCompleted OnWizardCompleted;

	/** Main wizard */
	TSharedPtr<SWizard> Wizard;

	/** Report Dialog used in first step (to display packages in a tree view) */
	TSharedPtr<SStormSyncReportDialog> ReportDialog;
	
	/** Initial list of selected assets in content browser that were selected by user (without dependencies) */
	TArray<FName> InitialPackageNames;

	/** Original list of selected assets in content browser (including dependencies) */
	TArray<FName> SelectedPackageNames;

	TSharedPtr<TArray<FStormSyncReportPackageData>> ReportPackages;

	/** The packages list view in destination page */
	TSharedPtr<SListView<TSharedPtr<FStormSyncFileDependency>>> FileDependenciesListView;

	/** File dependencies list */
	TArray<TSharedPtr<FStormSyncFileDependency>> FileDependencyList;

	/** The filename of the actual pak buffer (without extension) */
	FText ExportFilename;

	/** The path to place the file for the pak being generated */
	FText ExportDirectory;

	/** The editable text box to enter the current name */
	TSharedPtr<SEditableTextBox> FilenameEditBox;

	/** True if the last validity check returned that the class name/path is valid for creation */
	bool bLastInputValidityCheckSuccessful = false;

	/** The error text from the last validity check */
	FText LastInputValidityErrorText;

	/** Last column id that has been sorted */
	FName ColumnIdToSort;

	/** Current active sort mode */
	EColumnSortMode::Type ActiveSortMode = EColumnSortMode::Type::None;

	/** Handler for when the user enters the "choose file destination" page */
	void OnDestinationPageEntered();

	/** Filters out reports data for package names to be included and returns a flatten list of Package Names */
	TArray<FName> GetPackageNamesFromReportsData() const;

	/** Returns expected file size from selected package names to include */
	FText GetExpectedFileSize() const;

	/** Whether wizard can proceed to destination page */
	bool CanShowDestinationPage() const;

	/** Generate table row for file dependency list view */
	TSharedRef<ITableRow> MakeFileDependencyWidget(TSharedPtr<FStormSyncFileDependency> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Handler for cancel button, which closes the dialog */
	void OnCancelButtonClicked();

	/**
	 * Handler for finish wizard button, which closes the dialog and invokes OnWizardCompleted delegate with
	 * selected package names and destination path
	 */
	void OnFinish();

	/** Returns whether wizard can be completed */
	bool CanFinish() const;

	/** Returns the text in the file name edit box */
	FText GetFileNameText() const;

	/** Handler for when the text in the file name edit box has changed */
	void OnFileNameTextChanged(const FText& InNewText);

	/** Returns the text in the file path edit box */
	FText GetFilePathText() const;

	/** Handler for when the text in the file path edit box has changed */
	void OnFilePathTextChanged(const FText& InNewText);

	/** Handler for when the "Choose Folder" button is clicked */
	FReply HandleChooseFolderButtonClicked();

	/** Gets the visibility of the name error label */
	EVisibility GetNameErrorLabelVisibility() const;

	/** Gets the text to display in the name error label */
	FText GetNameErrorLabelText() const;

	/** Checks the current file name/path for validity and updates cached values accordingly */
	void UpdateInputValidity();

	/** Returns true if the project filename is properly formed and does not conflict with another project */
	static bool IsValidFilenameForCreation(const FString& InFilename, FText& OutFailReason);

	/** Checks the name for illegal characters */
	static bool NameContainsOnlyLegalCharacters(const FString& TestName, FString& OutIllegalCharacters);

	/** Returns last stored sort mode for column id if it matches last column being sorted */
	EColumnSortMode::Type GetSortModeForColumn(FName InColumnId) const;

	/** Updates both ColumnIdToSort and ActiveSortMode before performing the sort */
	void OnSortAttributeEntries(EColumnSortPriority::Type InPriority, const FName& InColumnId, EColumnSortMode::Type InSortMode);
};
