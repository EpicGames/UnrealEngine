// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SStormSyncFileDependencyWidgetRow.h"

#include "SlateOptMacros.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncImportTypes.h"
#include "Widgets/Text/STextBlock.h"

void SStormSyncFileDependencyWidgetRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	WidgetItem = InArgs._Item;

	SMultiColumnTableRow<TSharedPtr<FStormSyncFileDependency>>::Construct(
		FSuperRowType::FArguments().Padding(1.0f),
		InOwnerTableView
	);
}

TSharedRef<SWidget> SStormSyncFileDependencyWidgetRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	TSharedRef<SBox> BoxWrapper = SNew(SBox)
		.Padding(FMargin(4.0f, 0.0f))
		.VAlign(VAlign_Center);
	
	if (InColumnName == StormSync::SlateWidgetRow::HeaderRow_PackageName)
	{
		BoxWrapper->SetContent(SNew(STextBlock)
			.Text(FText::FromString(WidgetItem->FileDependency.PackageName.ToString()))
			.ToolTipText(FText::FromString(WidgetItem->FileDependency.ToString()))
		);
	}
	else if (InColumnName == StormSync::SlateWidgetRow::HeaderRow_FileSize)
	{
		BoxWrapper->SetContent(SNew(STextBlock)
			.Text(FText::FromString(FStormSyncCoreUtils::GetHumanReadableByteSize(WidgetItem->FileDependency.FileSize)))
			.ToolTipText(FText::FromString(FString::Printf(TEXT("%lld bytes"), WidgetItem->FileDependency.FileSize)))
		);
	}
	else if (InColumnName == StormSync::SlateWidgetRow::HeaderRow_Timestamp)
	{
		BoxWrapper->SetContent(SNew(STextBlock)
			.Text(FText::AsDateTime(FDateTime::FromUnixTimestamp(WidgetItem->FileDependency.Timestamp), EDateTimeStyle::Default))
			.ToolTipText(FText::FromString(FString::Printf(TEXT("Timestamp: %lld"), WidgetItem->FileDependency.Timestamp)))
		);
	}
	else if (InColumnName == StormSync::SlateWidgetRow::HeaderRow_FileHash)
	{
		const FString ShortHash = FString::Printf(TEXT("%s…"), *WidgetItem->FileDependency.FileHash.Mid(0, 16));
		BoxWrapper->SetContent(SNew(STextBlock)
			.Text(FText::FromString(ShortHash))
			.ToolTipText(FText::FromString(FString::Printf(TEXT("File Hash: %s"), *WidgetItem->FileDependency.FileHash)))
		);
	}
	else if (InColumnName == StormSync::SlateWidgetRow::HeaderRow_ImportReason)
	{
		BoxWrapper->SetContent(SNew(STextBlock)
			.Text(WidgetItem->ImportReason)
			.ToolTipText(WidgetItem->ImportReasonTooltip.IsEmpty() ? WidgetItem->ImportReason : WidgetItem->ImportReasonTooltip)
		);
	}

	return BoxWrapper;
}
