// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "StormSyncPackageDescriptor.h"

struct FStormSyncImportFileInfo;

namespace StormSync::SlateWidgetRow
{
	static FName HeaderRow_PackageName = TEXT("PackageName");
	static FName HeaderRow_FileSize = TEXT("FileSize");
	static FName HeaderRow_Timestamp = TEXT("Timestamp");
	static FName HeaderRow_FileHash = TEXT("FileHash");
	static FName HeaderRow_ImportReason = TEXT("ImportReason");

	static FText DefaultLabel_PackageName = NSLOCTEXT("StormSyncFileDependencyWidgetRow", "PackageNameColumnHeader", "Package Name");
	static FText DefaultLabel_FileSize = NSLOCTEXT("StormSyncFileDependencyWidgetRow", "FileSizeColumnHeader", "File Size");
	static FText DefaultLabel_Timestamp = NSLOCTEXT("StormSyncFileDependencyWidgetRow", "TimestampColumnHeader", "Timestamp");
	static FText DefaultLabel_FileHash = NSLOCTEXT("StormSyncFileDependencyWidgetRow", "FileHashColumnHeader", "File Hash");
	static FText DefaultLabel_ImportReason = NSLOCTEXT("StormSyncFileDependencyWidgetRow", "ImportReasonColumnHeader", "Import Reason");
}

/** Table Row for StormSyncFileDependency list */
class SStormSyncFileDependencyWidgetRow : public SMultiColumnTableRow<TSharedPtr<FStormSyncFileDependency>>
{
public:
	SLATE_BEGIN_ARGS(SStormSyncFileDependencyWidgetRow) {}
	
	SLATE_ARGUMENT(TSharedPtr<FStormSyncImportFileInfo>, Item)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs and OwnerTable */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	/** Generates appropriate widget for the given column name ID */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

private:
	/** The underlying item that the TableView is visualizing */
	TSharedPtr<FStormSyncImportFileInfo> WidgetItem;
};
