// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SStormSyncImportWizard.h"

#include "Interfaces/IMainFrameModule.h"
#include "Slate/SStormSyncFileDependencyWidgetRow.h"
#include "SlateOptMacros.h"
#include "StormSyncImportTypes.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Workflow/SWizard.h"

#define LOCTEXT_NAMESPACE "StormSyncImportWizard"

void SStormSyncImportWizard::Construct(const FArguments& InArgs, const TArray<FStormSyncImportFileInfo>& InFilesToImport, const TArray<FStormSyncImportFileInfo>& InBufferFiles)
{
	FilesToImport = InFilesToImport;
	BufferFiles = InBufferFiles;
	CurrentTab = EStormSyncImportWizardActiveTab::FilesToImport;

	InitListSources();
	
	ChildSlot
	[
		SAssignNew(Wizard, SWizard)
		.OnCanceled(this, &SStormSyncImportWizard::OnCancelButtonClicked)
		.OnFinished(this, &SStormSyncImportWizard::OnFinish)
		.CanFinish(this, &SStormSyncImportWizard::CanFinish)
		.FinishButtonText(LOCTEXT("Import", "Import"))
		.FinishButtonToolTip(this, &SStormSyncImportWizard::GetFinishButtonTooltip)
		.ShowPageList(false)
		
		// Main page - right now wizard has only one page
		+SWizard::Page()
		.CanShow(true)
		[
			// Populate the widget
			SNew(SBorder)
			.Padding(18)
			.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
			[
				SNew(SVerticalBox)
				
				// Page Description
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 8.f, 0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PageDescription_01", "You are currently trying to import files from a storm sync archive / pak. Here is the list of files that are about to be imported."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PageDescription_02", "When you click the \"Import\" button, files listed below will be imported to the local project, in the path they are associated with."))
				]

				// File List
				+SVerticalBox::Slot()
				.Padding(0.f, 16.f, 0.f, 0.f)
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
					.Text(LOCTEXT("Import_ListOfFiles_Label", "Files List"))
				]
				
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 8.f, 0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Import_ListOfFiles_Description", "Here is the list of files with different state (between local project and files in buffer), as well as the full list of files coming from buffer."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Import_ListOfFiles_Description_02", "You can display each list by clicking the button controls below."))
				]
				
				+SVerticalBox::Slot()
				.Padding(120.f, 8.f)
				.AutoHeight()
				[
					SNew(SSegmentedControl<EStormSyncImportWizardActiveTab>)
					.Value(CurrentTab)
					.OnValueChanged(this, &SStormSyncImportWizard::OnTabViewChanged)
					+SSegmentedControl<EStormSyncImportWizardActiveTab>::Slot(EStormSyncImportWizardActiveTab::FilesToImport)
					.Icon(FAppStyle::Get().GetBrush("Icons.SaveModified"))
					.Text(LOCTEXT("Tab_FilesToImport", "List of files to import"))
					.ToolTip(LOCTEXT("Tab_FilesToImport_Tooltip", "Click to see the list of files to import due to mismatched file state (File not existing locally or mismatched File Size or Hash)"))
					+SSegmentedControl<EStormSyncImportWizardActiveTab>::Slot(EStormSyncImportWizardActiveTab::BufferFiles)
					.Icon(FAppStyle::Get().GetBrush("Icons.Save"))
					.Text(LOCTEXT("Tab_FilesToImportFullList", "Full list of files from buffer"))
					.ToolTip(LOCTEXT("Tab_FilesToImportFullList_Tooltip", "Click to see the original list of files included in buffer"))
				]

				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.Padding(0.f, 8.f)
				[
					SAssignNew(TabContentSwitcher, SWidgetSwitcher)
					.WidgetIndex(0)
					+SWidgetSwitcher::Slot()
					[
						SAssignNew(ListViewFilesToImport, SListView<TSharedPtr<FStormSyncImportFileInfo>>)
						.ListItemsSource(&FilesToImportListSource)
						.HeaderRow(SNew(SHeaderRow)
							+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_PackageName)
							.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_PackageName)
							.SortMode(this, &SStormSyncImportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_PackageName)
							.OnSort(this, &SStormSyncImportWizard::OnSortAttributeEntries)
							.FillWidth(0.4f)
							+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_ImportReason)
							.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_ImportReason)
							.SortMode(this, &SStormSyncImportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_ImportReason)
							.OnSort(this, &SStormSyncImportWizard::OnSortAttributeEntries)
							.FillWidth(0.6f)
						)
						.OnGenerateRow(this, &SStormSyncImportWizard::MakeFileDependencyWidget)
					]
					 
					 +SWidgetSwitcher::Slot()
					 [
						SAssignNew(ListViewBufferFiles, SListView<TSharedPtr<FStormSyncImportFileInfo>>)
						.ListItemsSource(&BufferFilesListSource)
						.HeaderRow(SNew(SHeaderRow)
							+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_PackageName)
							.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_PackageName)
							.SortMode(this, &SStormSyncImportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_PackageName)
							.OnSort(this, &SStormSyncImportWizard::OnSortAttributeEntries)
							.FillWidth(0.4f)
							+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_FileSize)
							.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_FileSize)
							.SortMode(this, &SStormSyncImportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_FileSize)
							.OnSort(this, &SStormSyncImportWizard::OnSortAttributeEntries)
							.FillWidth(0.1f)
							+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_Timestamp)
							.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_Timestamp)
							.SortMode(this, &SStormSyncImportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_Timestamp)
							.OnSort(this, &SStormSyncImportWizard::OnSortAttributeEntries)
							.FillWidth(0.2f)
							+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_FileHash)
							.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_FileHash)
							.SortMode(this, &SStormSyncImportWizard::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_FileHash)
							.OnSort(this, &SStormSyncImportWizard::OnSortAttributeEntries)
							.FillWidth(0.2f)
						)
						.OnGenerateRow(this, &SStormSyncImportWizard::MakeFileDependencyWidget)
					 ]
				]
			]
		]
	];
}

void SStormSyncImportWizard::InitListSources()
{
	for (const FStormSyncImportFileInfo& FileInfo : FilesToImport)
	{
		FilesToImportListSource.AddUnique(MakeShared<FStormSyncImportFileInfo>(FileInfo));
	}
	
	for (const FStormSyncImportFileInfo& FileInfo : BufferFiles)
	{
		BufferFilesListSource.AddUnique(MakeShared<FStormSyncImportFileInfo>(FileInfo));
	}
}

bool SStormSyncImportWizard::ShouldImport() const
{
	return bShouldImport;
}

TSharedRef<ITableRow> SStormSyncImportWizard::MakeFileDependencyWidget(const TSharedPtr<FStormSyncImportFileInfo> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SStormSyncFileDependencyWidgetRow, OwnerTable).Item(InItem);
}

void SStormSyncImportWizard::OnCancelButtonClicked()
{
	CloseDialog();
}

void SStormSyncImportWizard::OnFinish()
{
	check(CanFinish());

	bShouldImport = true;
	
	CloseDialog();
}

bool SStormSyncImportWizard::CanFinish() const
{
	return !FilesToImport.IsEmpty();
}

void SStormSyncImportWizard::CloseDialog()
{
	const TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

void SStormSyncImportWizard::OnTabViewChanged(EStormSyncImportWizardActiveTab InActiveTab)
{
	CurrentTab = InActiveTab;

	if (InActiveTab == EStormSyncImportWizardActiveTab::FilesToImport)
	{
		TabContentSwitcher->SetActiveWidgetIndex(0);
	}
	else if (InActiveTab == EStormSyncImportWizardActiveTab::BufferFiles)
	{
		TabContentSwitcher->SetActiveWidgetIndex(1);
	}
}

FText SStormSyncImportWizard::GetFinishButtonTooltip() const
{
	return CanFinish() ?
		FText::Format(LOCTEXT("ImportButtonTooltip_Enabled", "Import {0} files to local project"), FText::AsNumber(FilesToImport.Num())) :
		LOCTEXT("ImportButtonTooltip_Disabled", "No Files to import");
}

EColumnSortMode::Type SStormSyncImportWizard::GetSortModeForColumn(const FName InColumnId) const
{
	if (ColumnIdToSort == InColumnId)
	{
		return ActiveSortMode;
	}
		
	return EColumnSortMode::None;
}

void SStormSyncImportWizard::OnSortAttributeEntries(EColumnSortPriority::Type InPriority, const FName& InColumnId, EColumnSortMode::Type InSortMode)
{
	ColumnIdToSort = InColumnId;
	ActiveSortMode = InSortMode;
	
	TArray<TSharedPtr<FStormSyncImportFileInfo>>* ListSource = TabContentSwitcher->GetActiveWidgetIndex() == 0 ? &FilesToImportListSource : &BufferFilesListSource;
	const TSharedPtr<STableViewBase> ListView = TabContentSwitcher->GetActiveWidgetIndex() == 0 ? ListViewFilesToImport : ListViewBufferFiles;

	ListSource->Sort([InColumnId, InSortMode](const TSharedPtr<FStormSyncImportFileInfo>& Left, const TSharedPtr<FStormSyncImportFileInfo>& Right)
	{		
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_FileSize)
		{
			return InSortMode == EColumnSortMode::Ascending ?
				Left->FileDependency.FileSize < Right->FileDependency.FileSize :
				Left->FileDependency.FileSize > Right->FileDependency.FileSize;
		}
		
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_Timestamp)
		{
			return InSortMode == EColumnSortMode::Ascending ?
				Left->FileDependency.Timestamp < Right->FileDependency.Timestamp :
				Left->FileDependency.Timestamp > Right->FileDependency.Timestamp;
		}
		
		int32 CompareResult = 0;
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_PackageName)
		{
			CompareResult = Left->FileDependency.PackageName.Compare(Right->FileDependency.PackageName);
		}

		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_FileHash)
		{
			CompareResult = Left->FileDependency.FileHash.Compare(Right->FileDependency.FileHash);
		}

		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_ImportReason)
		{
			CompareResult = Left->ImportReason.ToString().Compare(Right->ImportReason.ToString());
		}

		return InSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
	});

	ListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
