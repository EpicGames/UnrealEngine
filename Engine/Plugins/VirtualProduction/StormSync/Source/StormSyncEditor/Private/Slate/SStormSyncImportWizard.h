// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncImportWizard.h"
#include "StormSyncImportTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SWizard;
class SWidgetSwitcher;

/** Which tab page are we currently displaying */
enum class EStormSyncImportWizardActiveTab : uint8
{
	FilesToImport,
	BufferFiles
};

/**
 * A modal dialog to collect information about files to import in local project.
 *
 * It is wrapping a wizard with a single-page right now (could argue that it doesn't need to be a wizard in that case),
 * and a tab-like view with segmented control and widget switcher to display either the list of files to import (files
 * with mismatched state between local and buffer) or files in buffer.
 *
 * Note: UCommonTabListWidgetBase from CommonUI seems to implement a tab-like widget that would fit perfectly for that use case.
 */
class SStormSyncImportWizard : public SCompoundWidget, public IStormSyncImportWizard
{
public:
	SLATE_BEGIN_ARGS(SStormSyncImportWizard) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs, list of FStormSyncImportFileInfo to import and full list of FStormSyncImportFileInfo from buffer */
	void Construct(const FArguments& InArgs, const TArray<FStormSyncImportFileInfo>& InFilesToImport, const TArray<FStormSyncImportFileInfo>& InBufferFiles);

protected:
	//~ Begin IStormSyncImportWizard
	virtual bool ShouldImport() const override;
	//~ End IStormSyncImportWizard

private:
	/** Main wizard */
	TSharedPtr<SWizard> Wizard;
	
	/** The packages list view in files to import tab */
	TSharedPtr<SListView<TSharedPtr<FStormSyncImportFileInfo>>> ListViewFilesToImport;
	
	/** The packages list view in files to import tab */
	TSharedPtr<SListView<TSharedPtr<FStormSyncImportFileInfo>>> ListViewBufferFiles;
	
	/** File to import list view data source */
	TArray<TSharedPtr<FStormSyncImportFileInfo>> FilesToImportListSource;
	
	/** File to import list view data source */
	TArray<TSharedPtr<FStormSyncImportFileInfo>> BufferFilesListSource;

	/** Original list of file infos for files to import (mismatched state between buffer and local files) */
	TArray<FStormSyncImportFileInfo> FilesToImport;
	
	/** Original list of file infos included in buffer */
	TArray<FStormSyncImportFileInfo> BufferFiles;

	/** Current value for the segmented control active view */
	TAttribute<EStormSyncImportWizardActiveTab> CurrentTab;

	/** Switcher for active tab view */
	TSharedPtr<SWidgetSwitcher> TabContentSwitcher;

	/** Holds user choice on whether to perform import or cancel */
	bool bShouldImport = false;

	/** Last column id that has been sorted */
	FName ColumnIdToSort;

	/** Current active sort mode */
	EColumnSortMode::Type ActiveSortMode = EColumnSortMode::Type::None;

	/** Initializes sources for the list views from FilesToImport / BufferFiles */
	void InitListSources();
	
	/** Generate table row for file dependency list view */
	TSharedRef<ITableRow> MakeFileDependencyWidget(TSharedPtr<FStormSyncImportFileInfo> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;
	
	/** Handler for cancel button, which closes the dialog */
	void OnCancelButtonClicked();
	
	/** Handler for finish wizard button, which closes the dialog and sets up bShouldImport to true before exiting */
	void OnFinish();
	
	/** Returns whether wizard can be completed */
	bool CanFinish() const;

	/** Closes the dialog */
	void CloseDialog();

	/** Handler for segmented control tab view changed */
	void OnTabViewChanged(EStormSyncImportWizardActiveTab InActiveTab);

	/** Adjust finish button depending on wizard is able to finish (has files to import) */
	FText GetFinishButtonTooltip() const;

	/** Returns last stored sort mode for column id if it matches last column being sorted */
	EColumnSortMode::Type GetSortModeForColumn(FName InColumnId) const;

	/** Updates both ColumnIdToSort and ActiveSortMode before performing the sort */
	void OnSortAttributeEntries(EColumnSortPriority::Type InPriority, const FName& InColumnId, EColumnSortMode::Type InSortMode);
};
