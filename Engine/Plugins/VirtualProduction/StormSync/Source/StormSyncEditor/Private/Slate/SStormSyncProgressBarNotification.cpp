// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStormSyncProgressBarNotification.h"

#include "SlateOptMacros.h"
#include "StormSyncCoreUtils.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SStormSyncProgressBarNotification"

void SStormSyncProgressBarNotification::Construct(const FArguments& InArgs)
{
	TotalBytes = InArgs._TotalBytes;
	CurrentBytes = InArgs._CurrentBytes;
	EndpointAddress = InArgs._EndpointAddress;
	HostName = InArgs._HostName;
	OnDismissClicked = InArgs._OnDismissClicked;

	const TSharedPtr<SProgressBar> ProgressBar = SNew(SProgressBar)
		.Percent(this, &SStormSyncProgressBarNotification::GetProgressBarPercent)
		.FillColorAndOpacity(FSlateColor(FLinearColor(0.0f, 1.0f, 1.0f)));

	ChildSlot
	[
		SNew(SBox)
		.Padding(15.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(this, &SStormSyncProgressBarNotification::GetTextContent)
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontBold")))
			]
			+ SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("Sending_To", "Sending to {0} ({1})..."), FText::FromString(EndpointAddress), FText::FromString(HostName)))
				.Font(FAppStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 10.0f, 0.0f, 2.0f))
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					ProgressBar.ToSharedRef()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 10.0f, 0.0f, 2.0f))
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SHyperlink)
				.Text(LOCTEXT("Hyperlink_Dismiss_Text", "Dismiss"))
				.OnNavigate(this, &SStormSyncProgressBarNotification::OnHyperlinkClicked)
			]
		]
	];
}

float SStormSyncProgressBarNotification::GetPercent() const
{
	if (TotalBytes == 0)
	{
		return 0.f;
	}

	return static_cast<float>(CurrentBytes) / static_cast<float>(TotalBytes);
}

TOptional<float> SStormSyncProgressBarNotification::GetProgressBarPercent() const
{
	return GetPercent();
}

FText SStormSyncProgressBarNotification::GetTextContent() const
{
	if (TotalBytes == 0)
	{
		return LOCTEXT("Heading_Text_PreparingPak", "Preparing pak for transfer ...");
	}

	const FString Current = FStormSyncCoreUtils::GetHumanReadableByteSize(CurrentBytes);
	const FString Total = FStormSyncCoreUtils::GetHumanReadableByteSize(TotalBytes);

	return FText::FromString(FString::Printf(TEXT("%s / %s (%.02lf%%)"), *Current, *Total, GetPercent() * 100));
}

void SStormSyncProgressBarNotification::OnHyperlinkClicked()
{
	OnDismissClicked.ExecuteIfBound();
};

#undef LOCTEXT_NAMESPACE
