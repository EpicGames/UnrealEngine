// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Notifications/INotificationWidget.h"

/**
 * Simple widget to display progress bar when sending a pak buffer to a remote server.
 *
 * Designed to be used as the content of an SNotificationItem.
 */
class SStormSyncProgressBarNotification : public SCompoundWidget, public INotificationWidget
{
public:
	SLATE_BEGIN_ARGS(SStormSyncProgressBarNotification) {}

	/** Represents the total number of bytes for a buffer that is sent to a remote */
	SLATE_ARGUMENT(int32, TotalBytes)
	
	/** Represents the number of bytes received for a buffer on a remote */
	SLATE_ARGUMENT(int32, CurrentBytes)

	/** Represents the endpoint address for the remote we are sending to */
	SLATE_ARGUMENT(FString, EndpointAddress)
	
	/** Represents the hostname for the remote we are sending to */
	SLATE_ARGUMENT(FString, HostName)

	/** Invoked when dismiss link is clicked */
	SLATE_EVENT(FSimpleDelegate, OnDismissClicked)

	SLATE_END_ARGS()

	/** Native construct */
	void Construct(const FArguments& InArgs);

	//~ Begin INotificationWidget interface
	virtual void OnSetCompletionState(SNotificationItem::ECompletionState State) override {}
	virtual TSharedRef<SWidget> AsWidget() override
	{
		return AsShared();
	}
	//~ End INotificationWidget interface

	/** Updates CurrentBytes count */
	void SetCurrentBytes(const int32 InCurrentBytes)
	{
		CurrentBytes = InCurrentBytes;
	}
	
	/** Updates TotalBytes count */
	void SetTotalBytes(const int32 InTotalBytes)
	{
		TotalBytes = InTotalBytes;
	}

	/** Returns the actual progression based on TotalBytes and CurrentBytes, ranging from 0.f to 1.f */
	float GetPercent() const;

protected:
	/** Returns the actual progression based on TotalBytes and CurrentBytes, ranging from 0.f to 1.f */
	TOptional<float> GetProgressBarPercent() const;

	/**
	 * Returns text content for TextBlock heading of the widget
	 *
	 * If TotalBytes is set (not 0), will return CurrentBytes / TotalBytes in human readable form.
	 */
	FText GetTextContent() const;

	/** OnNavigate handler for the "dismiss" hyperlink */
	void OnHyperlinkClicked();
	
private:
	/** Represents the total number of bytes for a buffer that is sent to a remote */
	int32 TotalBytes = 0;
	
	/** Represents the number of bytes received for a buffer on a remote */
	int32 CurrentBytes = 0;
	
	/** Represents the endpoint address we are sending to */
	FString EndpointAddress;

	/** Represents the remote hostname we are sending to */
	FString HostName;
	
	/** Delegates for button clicks */
	FSimpleDelegate OnDismissClicked;
};
