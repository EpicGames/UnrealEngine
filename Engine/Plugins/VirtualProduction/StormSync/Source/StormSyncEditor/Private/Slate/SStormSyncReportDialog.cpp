// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SStormSyncReportDialog.h"

#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"

#define LOCTEXT_NAMESPACE "StormSyncPackageReportDialog"

struct FStormSyncCompareNodeByName
{
	FORCEINLINE bool operator()(TSharedPtr<FStormSyncPackageReportNode> A, TSharedPtr<FStormSyncPackageReportNode> B) const
	{
		return A->NodeName < B->NodeName;
	}
};

FString FStormSyncReportPackageData::ToString() const
{
	return FString::Printf(TEXT("Name: %s, bShouldMigratePackage: %s"), *Name, bShouldIncludePackage ? TEXT("true") : TEXT("false"));
}

void FStormSyncPackageReportNode::AddPackage(const FString& PackageName, bool* bInShouldMigratePackage)
{
	TArray<FString> PathElements;
	constexpr bool bCullEmpty = true;
	PackageName.ParseIntoArray(PathElements, TEXT("/"), bCullEmpty);

	AddPackage_Recursive(PathElements, bInShouldMigratePackage);
}

void FStormSyncPackageReportNode::ExpandChildrenRecursively(const TSharedRef<SStormSyncReportTreeView>& InTreeView)
{
	for (TSharedPtr<FStormSyncPackageReportNode> Child : Children)
	{
		InTreeView->SetItemExpansion(Child, Child->CheckedState != ECheckBoxState::Unchecked);
		Child->ExpandChildrenRecursively(InTreeView);
	}
}

FStormSyncPackageReportNode::FChildrenState FStormSyncPackageReportNode::AddPackage_Recursive(TArray<FString>& PathElements, bool* bInShouldMigratePackage)
{
	FChildrenState ChildrenState;
	ChildrenState.bAnyChildIsChecked = false;
	ChildrenState.bAllChildrenAreChecked = true;

	if (PathElements.Num() > 0)
	{
		// Pop the bottom element
		const FString ChildNodeName = PathElements[0];
		PathElements.RemoveAt(0);

		// Try to find a child which uses this folder name
		TSharedPtr<FStormSyncPackageReportNode> Child;
		for (auto ChildIt = Children.CreateConstIterator(); ChildIt; ++ChildIt)
		{
			if ((*ChildIt)->NodeName == ChildNodeName)
			{
				Child = (*ChildIt);
				break;
			}
		}

		// If one was not found, create it
		if (!Child.IsValid())
		{
			const bool bIsAFolder = (PathElements.Num() > 0);
			const int32 ChildIdx = Children.Add(MakeShareable(new FStormSyncPackageReportNode(ChildNodeName, bIsAFolder)));
			Child = Children[ChildIdx];
			Child.Get()->Parent = this;
			Children.Sort(FStormSyncCompareNodeByName());
		}

		if (ensure(Child.IsValid()))
		{
			const FChildrenState ChildChildrenState = Child->AddPackage_Recursive(PathElements, bInShouldMigratePackage);
			ChildrenState.bAnyChildIsChecked |= ChildChildrenState.bAnyChildIsChecked;
			ChildrenState.bAllChildrenAreChecked &= ChildChildrenState.bAllChildrenAreChecked;
		}

		CheckedState = ChildrenState.bAllChildrenAreChecked ? ECheckBoxState::Checked : (ChildrenState.bAnyChildIsChecked ? ECheckBoxState::Undetermined : ECheckBoxState::Unchecked);
	}
	else
	{
		CheckedState = *bInShouldMigratePackage ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		ChildrenState.bAnyChildIsChecked = ChildrenState.bAllChildrenAreChecked = CheckedState == ECheckBoxState::Checked;
		bShouldMigratePackage = bInShouldMigratePackage;
	}

	return ChildrenState;
}

void SStormSyncReportDialog::Construct(const FArguments& InArgs, const FText& InReportMessage, TArray<FStormSyncReportPackageData>& InPackageNames)
{
	FolderOpenBrush = FAppStyle::GetBrush("ContentBrowser.AssetTreeFolderOpen");
	FolderClosedBrush = FAppStyle::GetBrush("ContentBrowser.AssetTreeFolderClosed");
	PackageBrush = FAppStyle::GetBrush("ContentBrowser.ColumnViewAssetIcon");
	
	ConstructNodeTree(InPackageNames);
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
		.Padding(FMargin(4, 8, 4, 4))
		[
			SNew(SVerticalBox)

			// Report Message
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4)
			[
				SNew(STextBlock)
				.Text(InReportMessage)
				.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
			]

			// Tree of packages in the report
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(ReportTreeView, SStormSyncReportTreeView)
					.TreeItemsSource(&PackageReportRootNode.Children)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &SStormSyncReportDialog::GenerateTreeRow)
					.OnGetChildren_Static(&SStormSyncReportDialog::GetChildrenForTree)
				]
			]
		]
	];

	if (ensure(ReportTreeView.IsValid()))
	{
		PackageReportRootNode.ExpandChildrenRecursively(ReportTreeView.ToSharedRef());
	}
}

void SStormSyncReportDialog::CloseDialog()
{
	const TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (Window.IsValid())
	{
		Window->RequestDestroyWindow();
	}
}

void SStormSyncReportDialog::SetStateRecursive(TSharedPtr<FStormSyncPackageReportNode> TreeItem, bool bIsChecked)
{
	if (!TreeItem.IsValid())
	{
		return;
	}

	TreeItem->CheckedState = bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

	if (TreeItem->bShouldMigratePackage)
	{
		*(TreeItem->bShouldMigratePackage) = bIsChecked;
	}

	TArray<TSharedPtr<FStormSyncPackageReportNode>> Children;
	GetChildrenForTree(TreeItem, Children);
	
	for (const TSharedPtr<FStormSyncPackageReportNode>& Child : Children)
	{
		if (!Child.IsValid())
		{
			continue;
		}

		SetStateRecursive(Child, bIsChecked);
	}
}

ECheckBoxState SStormSyncReportDialog::GetEnabledCheckState(TSharedPtr<FStormSyncPackageReportNode> TreeItem)
{
	return TreeItem->CheckedState;
}

void SStormSyncReportDialog::CheckBoxStateChanged(ECheckBoxState InCheckBoxState, TSharedPtr<FStormSyncPackageReportNode> TreeItem, TSharedRef<STableViewBase> OwnerTable)
{
	SetStateRecursive(TreeItem, InCheckBoxState == ECheckBoxState::Checked);

	FStormSyncPackageReportNode* CurrentParent = TreeItem->Parent;
	while (CurrentParent != nullptr)
	{
		bool bAnyChildIsChecked = false;
		bool bAllChildrenAreChecked = true;

		for (const TSharedPtr<FStormSyncPackageReportNode>& Child : CurrentParent->Children)
		{
			bAnyChildIsChecked |= Child->CheckedState != ECheckBoxState::Unchecked;
			bAllChildrenAreChecked &= Child->CheckedState != ECheckBoxState::Unchecked;
		}

		CurrentParent->CheckedState = bAllChildrenAreChecked ? ECheckBoxState::Checked : (bAnyChildIsChecked ? ECheckBoxState::Undetermined : ECheckBoxState::Unchecked);
		CurrentParent = CurrentParent->Parent;
	}

	OwnerTable.Get().RebuildList();
}

void SStormSyncReportDialog::ConstructNodeTree(TArray<FStormSyncReportPackageData>& PackageNames)
{
	for (FStormSyncReportPackageData& Package : PackageNames)
	{
		PackageReportRootNode.AddPackage(Package.Name, &Package.bShouldIncludePackage);
	}
}

TSharedRef<ITableRow> SStormSyncReportDialog::GenerateTreeRow(TSharedPtr<FStormSyncPackageReportNode> TreeItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	check(TreeItem.IsValid());

	const FSlateBrush* IconBrush = GetNodeIcon(TreeItem);

	return SNew(STableRow<TSharedPtr<FStormSyncPackageReportNode>>, OwnerTable)
	[
		// Icon
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.OnCheckStateChanged_Static(&SStormSyncReportDialog::CheckBoxStateChanged, TreeItem, OwnerTable)
			.IsChecked_Static(&SStormSyncReportDialog::GetEnabledCheckState, TreeItem)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SImage)
			.Image(IconBrush)
		]
		// Name
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(TreeItem->NodeName))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
}

void SStormSyncReportDialog::GetChildrenForTree(const TSharedPtr<FStormSyncPackageReportNode> TreeItem, TArray<TSharedPtr<FStormSyncPackageReportNode>>& OutChildren)
{
	OutChildren = TreeItem->Children;
}

const FSlateBrush* SStormSyncReportDialog::GetNodeIcon(const TSharedPtr<FStormSyncPackageReportNode>& ReportNode) const
{
	if (!ReportNode->bIsFolder)
	{
		return PackageBrush;
	}
	
	if (ReportTreeView->IsItemExpanded(ReportNode))
	{
		return FolderOpenBrush;
	}
	
	return FolderClosedBrush;
}


#undef LOCTEXT_NAMESPACE
