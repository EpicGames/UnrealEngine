// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

struct FStormSyncPackageReportNode;
typedef STreeView<TSharedPtr<FStormSyncPackageReportNode>> SStormSyncReportTreeView;

/** Struct equivalent of ReportPackageData in Engine/Source/Developer/AssetTools/Private/SPackageReportDialog.h */
struct FStormSyncReportPackageData
{
	/** Package name, eg. /Game/Path/File */
	FString Name;

	/** Whether this package name is selected in the tree view */
	bool bShouldIncludePackage = false;

	FStormSyncReportPackageData(const FString& InName, const bool bInShouldMigratePackage)
		: Name(InName)
		, bShouldIncludePackage(bInShouldMigratePackage)
	{
	}

	FString ToString() const;
};

/** Struct equivalent of FPackageReportNode in Engine/Source/Developer/AssetTools/Private/SPackageReportDialog.h */
struct FStormSyncPackageReportNode
{
	/** The name of the tree node without the path */
	FString NodeName;

	/** A user-exposed flag determining whether the content of this node and its children should be migrated or not. */
	ECheckBoxState CheckedState;

	/** A pointer to an external bool describing whether this node ultimately should be migrated or not. Is only non-null for leaf nodes.*/
	bool* bShouldMigratePackage;

	/** If true, this node is a folder instead of a package */
	bool bIsFolder;

	/** The parent of this node */
	FStormSyncPackageReportNode* Parent;

	/** The children of this node */
	TArray<TSharedPtr<FStormSyncPackageReportNode>> Children;

	/** Constructor */
	FStormSyncPackageReportNode()
		: CheckedState(ECheckBoxState::Undetermined)
		, bShouldMigratePackage(nullptr)
		, bIsFolder(false)
		, Parent(nullptr)
	{
	}

	FStormSyncPackageReportNode(const FString& InNodeName, bool InIsFolder)
		: NodeName(InNodeName)
		, CheckedState(ECheckBoxState::Undetermined)
		, bShouldMigratePackage(nullptr)
		, bIsFolder(InIsFolder)
		, Parent(nullptr)
	{
	}

	/** Adds the path to the tree relative to this node, creating nodes as needed. */
	void AddPackage(const FString& PackageName, bool* bInShouldMigratePackage);

	/** Expands this node and all its children */
	void ExpandChildrenRecursively(const TSharedRef<SStormSyncReportTreeView>& InTreeView);

private:
	struct FChildrenState
	{
		bool bAnyChildIsChecked;
		bool bAllChildrenAreChecked;
	};

	/** Helper function for AddPackage. PathElements is the tokenized path delimited by "/" */
	FChildrenState AddPackage_Recursive(TArray<FString>& PathElements, bool* bInShouldMigratePackage);
};

/**
 * Straight up copy of Engine/Source/Developer/AssetTools/Private/SPackageReportDialog.h
 * which is used in Migrate Packages action.
 *
 * We can't use it directly as it's not exposed to outside modules.
 */
class SStormSyncReportDialog final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStormSyncReportDialog) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FText& InReportMessage, TArray<FStormSyncReportPackageData>& InPackageNames);

	/** Closes the dialog. */
	void CloseDialog();
	
private:
	FStormSyncPackageReportNode PackageReportRootNode;
	TSharedPtr<SStormSyncReportTreeView> ReportTreeView;

	/** Brushes for the different node states */
	const FSlateBrush* FolderOpenBrush = nullptr;
	const FSlateBrush* FolderClosedBrush = nullptr;
	const FSlateBrush* PackageBrush = nullptr;
	
	/** Recursively sets the checked/active state of every child of this node in the tree when a checkbox is toggled. */
	static void SetStateRecursive(TSharedPtr<FStormSyncPackageReportNode> TreeItem, bool bIsChecked);

	/** Callback to check whether a checkbox is checked or not. */
	static ECheckBoxState GetEnabledCheckState(TSharedPtr<FStormSyncPackageReportNode> TreeItem);

	/** Callback called whenever a checkbox is toggled. */
	static void CheckBoxStateChanged(ECheckBoxState InCheckBoxState, TSharedPtr<FStormSyncPackageReportNode> TreeItem, TSharedRef<STableViewBase> OwnerTable);

	/** Constructs the node tree given the list of package names */
	void ConstructNodeTree(TArray<FStormSyncReportPackageData>& PackageNames);

	/** Handler to generate a row in the report tree */
	TSharedRef<ITableRow> GenerateTreeRow(TSharedPtr<FStormSyncPackageReportNode> TreeItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Gets the children for the specified tree item */
	static void GetChildrenForTree(TSharedPtr<FStormSyncPackageReportNode> TreeItem, TArray<TSharedPtr<FStormSyncPackageReportNode>>& OutChildren);
	
	/** Determines which image to display next to a node */
	const FSlateBrush* GetNodeIcon(const TSharedPtr<FStormSyncPackageReportNode>& ReportNode) const;
};
