// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStormSyncStatusWidget.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "SlateOptMacros.h"
#include "Slate/SStormSyncFileDependencyWidgetRow.h"
#include "StormSyncEditorLog.h"
#include "StormSyncImportTypes.h"
#include "StormSyncTransportMessages.h"
#include "Widgets/SWindow.h"
#include "Widgets/Workflow/SWizard.h"

#define LOCTEXT_NAMESPACE "StormSyncStatusWidget"

void SStormSyncStatusWidget::Construct(const FArguments& InArgs, const TSharedRef<FStormSyncTransportStatusResponse>& InStatusResponse)
{
	FStormSyncTransportStatusResponse& StatusResponse = InStatusResponse.Get();
	UE_LOGF(LogStormSyncEditor, Display, "Construct StormSyncStatusWidget with %ls", *StatusResponse.ToString());

	const FStormSyncConnectionInfo RemoteInfo = InStatusResponse->ConnectionInfo;
	UE_LOGF(LogStormSyncEditor, Display, "\t Remote: %ls", *RemoteInfo.ToString());
	
	for (const FStormSyncFileModifierInfo& Modifier : StatusResponse.Modifiers)
	{
		UE_LOGF(LogStormSyncEditor, Display, "\t\t Modifier: %ls", *Modifier.ToString());

		FStormSyncImportFileInfo Info;

		// Slight repurpose of import reason to display modifier operation
		Info.ImportReason = UEnum::GetDisplayValueAsText(Modifier.ModifierOperation);
		Info.ImportReasonTooltip = GetModifierOperationTooltip(Modifier.ModifierOperation);
		Info.FileDependency = Modifier.FileDependency;

		ListSource.Add(MakeShared<FStormSyncImportFileInfo>(Info));
	}

	const TSharedRef<SWidget> StatusPanel = InStatusResponse->bNeedsSynchronization ? CreateModifiersListPanel() : CreateInSyncPanel();

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
		.Padding(18)
		[
			SAssignNew(Wizard, SWizard)
			.OnCanceled(this, &SStormSyncStatusWidget::OnCancelButtonClicked)
			.OnFinished(this, &SStormSyncStatusWidget::OnFinish)
			.CanFinish(this, &SStormSyncStatusWidget::CanFinish)
			.ShowPageList(false)

			// Main page
			+SWizard::Page()
			[
				SNew(SVerticalBox)
				
				// Page Description
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 8.f, 0.f, 4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PageDescription_01", "This is the result of a status request from the following remote instance:"))
				]

				// Remote information (TODO: Might rework in its standalone widget)
				+SVerticalBox::Slot()
				.Padding(16.f, 8.f, 0.f, 4.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						.Text(LOCTEXT("Remote_Hostname", "Remote Hostname:"))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.FillWidth(1.f)
					.Padding(4.f, 0.f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(RemoteInfo.HostName))
					]
				]
				
				+SVerticalBox::Slot()
				.Padding(16.f, 8.f, 0.f, 4.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						.Text(LOCTEXT("Remote_ProjectName", "Remote Project Name:"))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.FillWidth(1.f)
					.Padding(4.f, 0.f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(RemoteInfo.ProjectName))
					]
				]

				+SVerticalBox::Slot()
				.Padding(16.f, 8.f, 0.f, 4.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						.Text(LOCTEXT("Remote_ProjectDir", "Remote Project Directory:"))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.FillWidth(1.f)
					.Padding(4.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Remote_ProjectDir_Value", "Dir"))
						.Text(FText::FromString(RemoteInfo.ProjectDir))
					]
				]
				
				+SVerticalBox::Slot()
				.Padding(16.f, 8.f, 0.f, 4.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						.Text(LOCTEXT("Remote_InstanceId", "Remote Instance ID:"))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.FillWidth(1.f)
					.Padding(4.f, 0.f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(RemoteInfo.InstanceId.ToString()))
					]
				]

				+SVerticalBox::Slot()
				.Padding(16.f, 8.f, 0.f, 4.f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
						.Text(LOCTEXT("Remote_InstanceType", "Remote Instance Type:"))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.FillWidth(1.f)
					.Padding(4.f, 0.f)
					[
						SNew(STextBlock)
						.Text(UEnum::GetDisplayValueAsText(RemoteInfo.InstanceType))
					]
				]

				// Status Panel
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.Padding(0.f, 8.f)
				[
					StatusPanel
				]
			]
		]
	];
}

TSharedRef<SWindow> SStormSyncStatusWidget::CreateWindow(const TSharedRef<FStormSyncTransportStatusResponse>& InStatusResponse)
{
	return SNew(SWindow)
		.Title(LOCTEXT("Window_Title", "Storm Sync | Status Response"))
		.ClientSize(FVector2D(960, 700))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SStormSyncStatusWidget, InStatusResponse)
		];
}

void SStormSyncStatusWidget::OpenDialog(const TSharedRef<FStormSyncTransportStatusResponse>& InStatusResponse)
{
	const TSharedRef<SWindow> Window = CreateWindow(InStatusResponse);

	const IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	const TSharedPtr<SWindow> ParentWindow = MainFrameModule.GetParentWindow();

	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(Window, MainFrameModule.GetParentWindow().ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}
}

EColumnSortMode::Type SStormSyncStatusWidget::GetSortModeForColumn(const FName InColumnId) const
{
	if (ColumnIdToSort == InColumnId)
	{
		return ActiveSortMode;
	}
		
	return EColumnSortMode::None;
}

void SStormSyncStatusWidget::OnSortAttributeEntries(EColumnSortPriority::Type InPriority, const FName& InColumnId, EColumnSortMode::Type InSortMode)
{
	ColumnIdToSort = InColumnId;
	ActiveSortMode = InSortMode;
	
	ListSource.Sort([InColumnId, InSortMode](const TSharedPtr<FStormSyncImportFileInfo>& Left, const TSharedPtr<FStormSyncImportFileInfo>& Right)
	{		
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_FileSize)
		{
			return InSortMode == EColumnSortMode::Ascending ?
				Left->FileDependency.FileSize < Right->FileDependency.FileSize :
				Left->FileDependency.FileSize > Right->FileDependency.FileSize;
		}
		
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_Timestamp)
		{
			return InSortMode == EColumnSortMode::Ascending ?
				Left->FileDependency.Timestamp < Right->FileDependency.Timestamp :
				Left->FileDependency.Timestamp > Right->FileDependency.Timestamp;
		}
		
		int32 CompareResult = 0;
		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_PackageName)
		{
			CompareResult = Left->FileDependency.PackageName.Compare(Right->FileDependency.PackageName);
		}

		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_FileHash)
		{
			CompareResult = Left->FileDependency.FileHash.Compare(Right->FileDependency.FileHash);
		}

		if (InColumnId == StormSync::SlateWidgetRow::HeaderRow_ImportReason)
		{
			CompareResult = Left->ImportReason.ToString().Compare(Right->ImportReason.ToString());
		}

		return InSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
	});

	ListView->RequestListRefresh();
}

TSharedRef<ITableRow> SStormSyncStatusWidget::MakeFileDependencyWidget(const TSharedPtr<FStormSyncImportFileInfo> InItem, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(SStormSyncFileDependencyWidgetRow, OwnerTable).Item(InItem);
}

FText SStormSyncStatusWidget::GetModifierOperationTooltip(EStormSyncModifierOperation InModifierOperation)
{
	switch (InModifierOperation)
	{
		case EStormSyncModifierOperation::Addition:
			return LOCTEXT("Operation_Addition_Description", "This is an addition, meaning remote is missing the file.");
		case EStormSyncModifierOperation::Missing:
			return LOCTEXT("Operation_Missing_Description", "This is a missing situation, meaning remote has the file but local has not.");
		case EStormSyncModifierOperation::Overwrite:
			return LOCTEXT("Operation_Overwrite_Description", "This is an overwrite, meaning both remote and local have the file, but in a different state (mismatch filesize, hash, etc.)");
		default: 
			UE_LOGF(LogStormSyncEditor, Warning, "SStormSyncStatusWidget::GetModifierOperationTooltip: Invalid operation %d", static_cast<int32>(InModifierOperation));
			return FText::GetEmpty();
	}
}

TSharedRef<SWidget> SStormSyncStatusWidget::CreateModifiersListPanel()
{
	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 8.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("File_Diff_Description", "Here is the list of files with different states (between local project and remote)."))
		]
		
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(0.f, 8.f)
		[
			SAssignNew(ListView, SListView<TSharedPtr<FStormSyncImportFileInfo>>)
			.ListItemsSource(&ListSource)
			.HeaderRow(SNew(SHeaderRow)
				+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_PackageName)
				.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_PackageName)
				.SortMode(this, &SStormSyncStatusWidget::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_PackageName)
				.OnSort(this, &SStormSyncStatusWidget::OnSortAttributeEntries)
				.FillWidth(0.3f)
				+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_ImportReason)
				// Slightly repurpose import reason to display modifier operation
				.DefaultLabel(LOCTEXT("MimatchReason", "Mismatch Reason"))
				.SortMode(this, &SStormSyncStatusWidget::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_ImportReason)
				.OnSort(this, &SStormSyncStatusWidget::OnSortAttributeEntries)
				.FillWidth(0.2f)
				+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_FileSize)
				.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_FileSize)
				.SortMode(this, &SStormSyncStatusWidget::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_FileSize)
				.OnSort(this, &SStormSyncStatusWidget::OnSortAttributeEntries)
				.FillWidth(0.1f)
				+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_Timestamp)
				.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_Timestamp)
				.SortMode(this, &SStormSyncStatusWidget::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_Timestamp)
				.OnSort(this, &SStormSyncStatusWidget::OnSortAttributeEntries)
				.FillWidth(0.2f)
				+SHeaderRow::Column(StormSync::SlateWidgetRow::HeaderRow_FileHash)
				.DefaultLabel(StormSync::SlateWidgetRow::DefaultLabel_FileHash)
				.SortMode(this, &SStormSyncStatusWidget::GetSortModeForColumn, StormSync::SlateWidgetRow::HeaderRow_FileHash)
				.OnSort(this, &SStormSyncStatusWidget::OnSortAttributeEntries)
				.FillWidth(0.2f)
			)
			.OnGenerateRow(this, &SStormSyncStatusWidget::MakeFileDependencyWidget)
		];
}

TSharedRef<SWidget> SStormSyncStatusWidget::CreateInSyncPanel()
{
	return SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	.Padding(32.f)
	.HAlign(HAlign_Center)
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2d(48.f))
			.Image(FAppStyle::GetBrush("Icons.Check"))
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8.f, 16.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("Font.Large.Bold"))
			.Text(LOCTEXT("Status_InSync", "In Sync!"))
		]
	];
}

void SStormSyncStatusWidget::OnCancelButtonClicked()
{
	CloseDialog();
}

void SStormSyncStatusWidget::OnFinish()
{
	check(CanFinish());
	CloseDialog();
}

bool SStormSyncStatusWidget::CanFinish() const
{
	return true;
}

void SStormSyncStatusWidget::CloseDialog()
{
	const TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
