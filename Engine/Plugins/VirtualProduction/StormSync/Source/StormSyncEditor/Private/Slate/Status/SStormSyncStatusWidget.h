// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StormSyncPackageDescriptor.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SWizard;
struct FStormSyncImportFileInfo;
struct FStormSyncTransportStatusResponse;

/** A modal dialog to show the status of package names file state between two editor instances  */
class SStormSyncStatusWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStormSyncStatusWidget) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<FStormSyncTransportStatusResponse>& InStatusResponse);

	/** Create a new SWindow with a single status widget child */
	static TSharedRef<SWindow> CreateWindow(const TSharedRef<FStormSyncTransportStatusResponse>& InStatusResponse);

	/** Creates and opens a new status widget window */
	static void OpenDialog(const TSharedRef<FStormSyncTransportStatusResponse>& InStatusResponse);

private:
	
	/**
	 * Main wizard. Using a wizard widget here even though it's technically not a wizard (only one page)
	 *
	 * Doing this for visual consistency with other widgets, as well as the buttons utility we get by default with wizards (to close the widget)
	 */
	TSharedPtr<SWizard> Wizard;
	
	/** The packages list view in files to import tab */
	TSharedPtr<SListView<TSharedPtr<FStormSyncImportFileInfo>>> ListView;
	
	/** File to import list view data source */
	TArray<TSharedPtr<FStormSyncImportFileInfo>> ListSource;
	
	/** Last column id that has been sorted */
	FName ColumnIdToSort;

	/** Current active sort mode */
	EColumnSortMode::Type ActiveSortMode = EColumnSortMode::Type::None;

	/** Returns last stored sort mode for column id if it matches last column being sorted */
	EColumnSortMode::Type GetSortModeForColumn(FName InColumnId) const;

	/** Updates both ColumnIdToSort and ActiveSortMode before performing the sort */
	void OnSortAttributeEntries(EColumnSortPriority::Type InPriority, const FName& InColumnId, EColumnSortMode::Type InSortMode);
	
	/** Generate table row for file dependency list view */
	TSharedRef<ITableRow> MakeFileDependencyWidget(TSharedPtr<FStormSyncImportFileInfo> InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

	/** Returns the display text for a given modifier operation */
	static FText GetModifierOperationTooltip(EStormSyncModifierOperation InModifierOperation);

	/** Create widget panel showing list of modifiers (if response bNeedsSynchronization is true) */
	TSharedRef<SWidget> CreateModifiersListPanel();
	
	/** Create widget panel showing the "In Sync" text (if response bNeedsSynchronization is false) */
	static TSharedRef<SWidget> CreateInSyncPanel();
	
	/** Handler for cancel button, which closes the dialog */
	void OnCancelButtonClicked();
	
	/** Handler for finish wizard button, which simply closes the dialog */
	void OnFinish();
	
	/** Returns whether wizard can be completed */
	bool CanFinish() const;
	
	/** Closes the dialog */
	void CloseDialog();
};
