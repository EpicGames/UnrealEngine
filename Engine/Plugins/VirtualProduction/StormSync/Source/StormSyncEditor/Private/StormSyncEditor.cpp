// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncEditor.h"

#include "AssetRegistry/AssetData.h"
#include "ContextMenu/StormSyncAssetFolderContextMenu.h"
#include "Customization/StormSyncTransportSettingsDetailsCustomization.h"
#include "Framework/Application/SlateApplication.h"
#include "IStormSyncTransportClientModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "PropertyEditorModule.h"
#include "Slate/SStormSyncImportWizard.h"
#include "Slate/Status/SStormSyncStatusWidget.h"
#include "StormSyncCommandLineUtils.h"
#include "StormSyncEditorLog.h"
#include "StormSyncTransportSettings.h"

#define LOCTEXT_NAMESPACE "StormSyncEditor"

void FStormSyncEditorModule::StartupModule()
{
	// Register to storm sync service discovery related events
	FStormSyncCoreDelegates::OnServiceDiscoveryConnection.AddRaw(this, &FStormSyncEditorModule::OnServiceDiscoveryConnection);
	FStormSyncCoreDelegates::OnServiceDiscoveryStateChange.AddRaw(this, &FStormSyncEditorModule::OnServiceDiscoveryStateChange);
	FStormSyncCoreDelegates::OnServiceDiscoveryServerStatusChange.AddRaw(this, &FStormSyncEditorModule::OnServiceDiscoveryServerStatusChange);
	FStormSyncCoreDelegates::OnServiceDiscoveryDisconnection.AddRaw(this, &FStormSyncEditorModule::OnServiceDiscoveryDisconnection);

	// Create the asset menu instances
	AssetFolderContextMenu = MakeShared<FStormSyncAssetFolderContextMenu>();
	AssetFolderContextMenu->Initialize();

	// Details customization for IP Dropdown in settings
	RegisterDetailsCustomizations();
	
	if (GIsEditor && !IsRunningCommandlet())
	{
		RegisterConsoleCommands();
	}
}

void FStormSyncEditorModule::ShutdownModule()
{
	// Cleanup delegates
	FStormSyncCoreDelegates::OnServiceDiscoveryConnection.RemoveAll(this);
	FStormSyncCoreDelegates::OnServiceDiscoveryStateChange.RemoveAll(this);
	FStormSyncCoreDelegates::OnServiceDiscoveryServerStatusChange.RemoveAll(this);
	FStormSyncCoreDelegates::OnServiceDiscoveryDisconnection.RemoveAll(this);

	if (AssetFolderContextMenu.IsValid())
	{
		AssetFolderContextMenu->Shutdown();
		AssetFolderContextMenu.Reset();
	}

	// Cleanup Details customizations
	UnregisterDetailsCustomizations();

	// Cleanup commands
	UnregisterConsoleCommands();
}

TSharedRef<IStormSyncImportWizard> FStormSyncEditorModule::CreateWizard(const TArray<FStormSyncImportFileInfo>& InFilesToImport, const TArray<FStormSyncImportFileInfo>& InBufferFiles)
{
	const TSharedRef<SStormSyncImportWizard> ImportWizard = SNew(SStormSyncImportWizard, InFilesToImport, InBufferFiles);

	const TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("Window_Title", "Storm Sync | Import files from local pak"))
		.ClientSize(FVector2D(960, 700))
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			ImportWizard->AsShared()
		];

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		const IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
	return ImportWizard;
}

void FStormSyncEditorModule::BuildPushAssetsMenuSection(FMenuBuilder& InMenuBuilder, const TArray<FName> InPackageNames, const bool bInIsPushing) const
{
	// Exposed to outside modules so that they're able to generate this submenu (useful for UI extensions)
	if (AssetFolderContextMenu.IsValid())
	{
		AssetFolderContextMenu->BuildPushAssetsMenuSection(InMenuBuilder, InPackageNames, bInIsPushing);
	}
}

void FStormSyncEditorModule::BuildCompareWithMenuSection(FMenuBuilder& InMenuBuilder, const TArray<FName> InPackageNames) const
{
	// Exposed to outside modules so that they're able to generate this submenu (useful for UI extensions)
	if (AssetFolderContextMenu.IsValid())
	{
		AssetFolderContextMenu->BuildCompareWithMenuSection(InMenuBuilder, InPackageNames);
	}
}

TArray<FAssetData> FStormSyncEditorModule::GetDirtyAssets(const TArray<FName>& InPackageNames, FText& OutDisabledReason) const
{
	// Exposed to outside modules so that they're able to generate submenus (useful for UI extensions)
	TArray<FAssetData> AssetsData;
	
	if (AssetFolderContextMenu.IsValid())
	{
		AssetsData = AssetFolderContextMenu->GetDirtyAssets(InPackageNames, OutDisabledReason);
	}

	return AssetsData;
}

TMap<FMessageAddress, FStormSyncConnectedDevice> FStormSyncEditorModule::GetRegisteredConnections()
{
	FScopeLock ConnectionsLock(&ConnectionsCriticalSection);
	return RegisteredConnections;
}

void FStormSyncEditorModule::RegisterDetailsCustomizations() const
{
	// Register detail customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(
		UStormSyncTransportSettings::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FStormSyncTransportSettingsDetailsCustomization::MakeInstance)
	);
}

void FStormSyncEditorModule::UnregisterDetailsCustomizations() const
{
	if (!UObjectInitialized())
	{
		return;
	}

	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyEditorModule->UnregisterCustomClassLayout(UStormSyncTransportSettings::StaticClass()->GetFName());
	}
}

void FStormSyncEditorModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Debug.DumpConnections"),
		TEXT("Dump to console state of registered connections (service discovery related)"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncEditorModule::ExecuteDumpConnections),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Debug.RequestStatus"),
		TEXT("Sends a status request and outputs sync state for a remote. Usage: <PackageNames>... [-remote=<AddressId>]"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncEditorModule::ExecuteRequestStatusCommand),
		ECVF_Default
	));
}

void FStormSyncEditorModule::ExecuteDumpConnections(const TArray<FString>& Args)
{
	FScopeLock ConnectionsLock(&ConnectionsCriticalSection);
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::ExecuteDumpConnections - Listing active connections (%d)", RegisteredConnections.Num());
	
	for (const TPair<FMessageAddress, FStormSyncConnectedDevice>& RegisteredConnection : RegisteredConnections)
	{
		FMessageAddress MessageAddress = RegisteredConnection.Key;
		FStormSyncConnectedDevice Connection = RegisteredConnection.Value;

		FText Tooltip = GetEntryTooltipForRemote(MessageAddress, Connection);

		FString ConnectionDescription = FString::Printf(TEXT("\n\n%s\n\n"), *Tooltip.ToString());
		UE_LOGF(LogStormSyncEditor, Display, "%ls (%ls) %ls", *Connection.StormSyncServerAddressId, *Connection.HostName, *ConnectionDescription);
	}
}

void FStormSyncEditorModule::UnregisterConsoleCommands()
{
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}

	ConsoleCommands.Empty();
}

void FStormSyncEditorModule::ExecuteRequestStatusCommand(const TArray<FString>& Args) const
{
	const FString Argv = FString::Join(Args, TEXT(" "));
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::ExecuteStatusCommand - Argv: %ls", *Argv);

	// Parse command line.
	TArray<FName> PackageNames;
	FStormSyncCommandLineUtils::Parse(*Argv, PackageNames);

	if (PackageNames.IsEmpty())
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncEditorModule::ExecuteStatusCommand - Missing at least one package name to request status.");
		return;
	}

	FString RemoteAddressId;
	FParse::Value(*Argv, TEXT("-remote="), RemoteAddressId);
	if (RemoteAddressId.IsEmpty())
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncEditorModule::ExecuteStatusCommand - Missing -remote parameter");
		return;
	}

	FMessageAddress RemoteAddress;
	if (!FMessageAddress::Parse(RemoteAddressId, RemoteAddress))
	{
		UE_LOGF(LogStormSyncEditor, Error, "FStormSyncEditorModule::ExecuteStatusCommand - Unable to parse %ls into a Message Address", *RemoteAddressId);
		return;
	}

	const FOnStormSyncRequestStatusComplete DoneDelegate = FOnStormSyncRequestStatusComplete::CreateLambda([](const TSharedPtr<FStormSyncTransportStatusResponse>& Response)
	{
		UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::ExecuteStatusCommand - Received response! %ls", *Response->ToString());
		SStormSyncStatusWidget::OpenDialog(Response.ToSharedRef());
	});
	
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::ExecuteStatusCommand - Send status request on %ls", *RemoteAddress.ToString());
	IStormSyncTransportClientModule::Get().RequestPackagesStatus(RemoteAddress, PackageNames, DoneDelegate);
}

void FStormSyncEditorModule::OnServiceDiscoveryConnection(const FString& MessageAddressUID, const FStormSyncConnectedDevice& ConnectedDevice)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::OnServiceDiscoveryConnection - MessageAddressUID: %ls, Hostname: %ls, ProjectName: %ls", *MessageAddressUID, *ConnectedDevice.HostName, *ConnectedDevice.ProjectName);
	
	FScopeLock ConnectionsLock(&ConnectionsCriticalSection);

	FMessageAddress RemoteMessageAddress;
	if (FMessageAddress::Parse(MessageAddressUID, RemoteMessageAddress) && !RegisteredConnections.Contains(RemoteMessageAddress))
	{
		RegisteredConnections.Add(RemoteMessageAddress, ConnectedDevice);
	}
}

void FStormSyncEditorModule::OnServiceDiscoveryStateChange(const FString& MessageAddressUID, EStormSyncConnectedDeviceState State)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::OnServiceDiscoveryStateChange - MessageAddressUID: %ls", *MessageAddressUID);
	
	FScopeLock ConnectionsLock(&ConnectionsCriticalSection);
	
	FMessageAddress RemoteMessageAddress;
	if (FMessageAddress::Parse(MessageAddressUID, RemoteMessageAddress) && RegisteredConnections.Contains(RemoteMessageAddress))
	{
		UE_LOGF(LogStormSyncEditor, Display, "\tRemote State Changed to %ls", *UEnum::GetValueAsString(State));
		FStormSyncConnectedDevice& Connection = RegisteredConnections.FindChecked(RemoteMessageAddress);
		Connection.State = State;
	}
}

void FStormSyncEditorModule::OnServiceDiscoveryServerStatusChange(const FString& MessageAddressUID, bool bIsServerRunning)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::OnServiceDiscoveryServerStatusChange - MessageAddressUID: %ls", *MessageAddressUID);
	
	FScopeLock ConnectionsLock(&ConnectionsCriticalSection);
	
	FMessageAddress RemoteMessageAddress;
	if (FMessageAddress::Parse(MessageAddressUID, RemoteMessageAddress) && RegisteredConnections.Contains(RemoteMessageAddress))
	{
		UE_LOGF(LogStormSyncEditor, Display, "\tRemote Server Endpoint State Changed to %ls", bIsServerRunning ? TEXT("running") : TEXT("stopped"));
		FStormSyncConnectedDevice& Connection = RegisteredConnections.FindChecked(RemoteMessageAddress);
		Connection.bIsServerRunning = bIsServerRunning;
	}
}

void FStormSyncEditorModule::OnServiceDiscoveryDisconnection(const FString& MessageAddressUID)
{
	UE_LOGF(LogStormSyncEditor, Display, "FStormSyncEditorModule::OnServiceDiscoveryDisconnection - MessageAddressUID: %ls", *MessageAddressUID);
	
	FScopeLock ConnectionsLock(&ConnectionsCriticalSection);
	
	FMessageAddress RemoteMessageAddress;
	if (FMessageAddress::Parse(MessageAddressUID, RemoteMessageAddress) && RegisteredConnections.Contains(RemoteMessageAddress))
	{
		RegisteredConnections.Remove(RemoteMessageAddress);
	}
}

FText FStormSyncEditorModule::GetEntryTooltipForRemote(const FMessageAddress& InRemoteAddress, const FStormSyncConnectedDevice& InConnection)
{
	return FText::FromString(FString::Printf(
		TEXT("State: %s\nAddress ID: %s\nServer State: %s\nStorm Sync Server Address ID: %s,\nStorm Sync Client Address ID: %s,\nHostname: %s\nInstance Type: %s\nProject Name: %s\nProject Directory: %s"),
		*UEnum::GetValueAsString(InConnection.State),
		*InRemoteAddress.ToString(),
		InConnection.bIsServerRunning ? TEXT("Running") : TEXT("Stopped"),
		*InConnection.StormSyncServerAddressId,
		*InConnection.StormSyncClientAddressId,
		*InConnection.HostName,
		*UEnum::GetValueAsString(InConnection.InstanceType),
		*InConnection.ProjectName,
		*InConnection.ProjectDir
	));
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FStormSyncEditorModule, StormSyncEditor)
