// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/StormSyncNotificationSubsystem.h"

#include "Async/Async.h"
#include "Editor.h"
#include "MessageLogModule.h"
#include "Modules/ModuleManager.h"
#include "Slate/SStormSyncProgressBarNotification.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncEditorLog.h"
#include "StormSyncTransportMessages.h"

#define LOCTEXT_NAMESPACE "StormSyncNotificationSubsystem"

void UStormSyncNotificationSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncNotificationSubsystem initialized");

	// Create a message log for the notifications to use
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = true;
	InitOptions.bAllowClear = true;
	InitOptions.bDiscardDuplicates = false;
	InitOptions.bShowInLogWindow = true;
	InitOptions.bScrollToBottom = true;
	MessageLogModule.RegisterLogListing(LogName, LOCTEXT("StormSyncLogLabel", "Storm Sync Notifications"), InitOptions);

	MessageLog = MakeUnique<FMessageLog>(LogName);

	FStormSyncCoreDelegates::OnReceivingBytes.AddUObject(this, &UStormSyncNotificationSubsystem::OnReceivingBytes);
	FStormSyncCoreDelegates::OnStartSendingBuffer.AddUObject(this, &UStormSyncNotificationSubsystem::OnStartSendingBuffer);
	FStormSyncCoreDelegates::OnPreStartSendingBuffer.AddUObject(this, &UStormSyncNotificationSubsystem::OnPreStartSendingBuffer);
}

void UStormSyncNotificationSubsystem::Deinitialize()
{
	Super::Deinitialize();
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncNotificationSubsystem shutdown");

	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		// unregister message log
		FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing(LogName);
	}

	if (MessageLog.IsValid())
	{
		MessageLog.Reset();
	}

	FStormSyncCoreDelegates::OnReceivingBytes.RemoveAll(this);
	FStormSyncCoreDelegates::OnStartSendingBuffer.RemoveAll(this);
	FStormSyncCoreDelegates::OnPreStartSendingBuffer.RemoveAll(this);

	Notifications.Empty();
}

UStormSyncNotificationSubsystem& UStormSyncNotificationSubsystem::Get()
{
	check(GEditor);
	return *GEditor->GetEditorSubsystem<UStormSyncNotificationSubsystem>();
}

TSharedPtr<SNotificationItem> UStormSyncNotificationSubsystem::AddSimpleNotification(const FText& InNotificationText)
{
	const FNotificationInfo NotifyInfo(InNotificationText);
	return FSlateNotificationManager::Get().AddNotification(NotifyInfo);
}

TSharedPtr<SNotificationItem> UStormSyncNotificationSubsystem::AddSimpleNotification(const FNotificationInfo& InInfo)
{
	return FSlateNotificationManager::Get().AddNotification(InInfo);
}

void UStormSyncNotificationSubsystem::NewPage(const FText& InLabel, const bool bInAppendDateTime)
{
	check(MessageLog.IsValid());
	
	if (bInAppendDateTime)
	{
		const FText PageLabel = FText::Format(LOCTEXT("PageLabel_DateTime_Format", "{0} - {1}"), InLabel, FText::AsDateTime(FDateTime::UtcNow()));
		MessageLog->NewPage(PageLabel);
	}
	else
	{
		MessageLog->NewPage(InLabel);
	}
}

void UStormSyncNotificationSubsystem::Info(const FText& InMessageLogText)
{
	check(MessageLog.IsValid());
	MessageLog->Info(InMessageLogText);
}

void UStormSyncNotificationSubsystem::Warning(const FText& InMessageLogText)
{
	check(MessageLog.IsValid());
	MessageLog->Warning(InMessageLogText);
}

void UStormSyncNotificationSubsystem::Error(const FText& InMessageLogText)
{
	check(MessageLog.IsValid());
	MessageLog->Error(InMessageLogText);
}

void UStormSyncNotificationSubsystem::Notify(const EMessageSeverity::Type InSeverity, const FText& InMessageLogText, const bool bInForce)
{
	const FName LocalLogName = LogName;
	const FText LocalMessageLogText = InMessageLogText;

	// Ensures notification always run in a game thread
	AsyncTask(ENamedThreads::GameThread, [LocalLogName, InSeverity, LocalMessageLogText, bInForce]()
	{
		FMessageLog LocalMessageLog(LocalLogName);
		LocalMessageLog.Notify(LocalMessageLogText, InSeverity, bInForce);
	});
}

void UStormSyncNotificationSubsystem::HandlePushResponse(const TSharedPtr<FStormSyncTransportPushResponse>& InResponse)
{
	check(InResponse.IsValid())
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncNotificationSubsystem::HandlePushResponse - Response message: %ls", *InResponse->ToString());

	HandleSyncResponse(
		InResponse,
		LOCTEXT("Success_PushAssets", "Succesfully pushed to\n\nHostname: {0}\nProject: {1}\nDirectory: {2}\nInstance Type: {3}\n\n{4}"),
		LOCTEXT("Error_PushAssets", "Error pushing to\n\nHostname: {0}\nProject: {1}\nDirectory: {2}\nInstance Type: {3}\n\n{4}")
	);
}

void UStormSyncNotificationSubsystem::HandlePullResponse(const TSharedPtr<FStormSyncTransportPullResponse>& InResponse)
{
	check(InResponse.IsValid())
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncNotificationSubsystem::HandlePullResponse - Response message: %ls", *InResponse->ToString());

	HandleSyncResponse(
		InResponse,
		LOCTEXT("Success_PullAssets", "Succesfully pulled from\n\nHostname: {0}\nProject: {1}\nDirectory: {2}\nInstance Type: {3}\n\n{4}"),
		LOCTEXT("Error_PullAssets", "Error pulling from\n\nHostname: {0}\nProject: {1}\nDirectory: {2}\nInstance Type: {3}\n\n{4}")
	);
}

void UStormSyncNotificationSubsystem::HandleSyncResponse(const TSharedPtr<FStormSyncTransportSyncResponse>& InResponse, const FText& InSuccessTextFormat, const FText& InErrorTextFormat)
{
	check(InResponse.IsValid())
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncNotificationSubsystem::HandleSyncResponse - Response message: %ls", *InResponse->ToString());

	FText NotifyMessage;
	EMessageSeverity::Type Severity = EMessageSeverity::Info;
	if (InResponse->Status == EStormSyncResponseResult::Success)
	{
		Severity = EMessageSeverity::Info;
		NotifyMessage = FText::Format(
			InSuccessTextFormat,
			FText::FromString(InResponse->ConnectionInfo.HostName),
			FText::FromString(InResponse->ConnectionInfo.ProjectName),
			FText::FromString(InResponse->ConnectionInfo.ProjectDir),
			UEnum::GetDisplayValueAsText(InResponse->ConnectionInfo.InstanceType),
			InResponse->StatusText
		);

		Info(InResponse->StatusText);
	}
	else if (InResponse->Status == EStormSyncResponseResult::Error)
	{
		Severity = EMessageSeverity::Error;
		NotifyMessage = FText::Format(
			InErrorTextFormat,
			FText::FromString(InResponse->ConnectionInfo.HostName),
			FText::FromString(InResponse->ConnectionInfo.ProjectName),
			FText::FromString(InResponse->ConnectionInfo.ProjectDir),
			UEnum::GetDisplayValueAsText(InResponse->ConnectionInfo.InstanceType),
			InResponse->StatusText
		);

		Error(InResponse->StatusText);
	}
	else
	{
		checkf(false, TEXT("Got a response with an invalid status"));
	}

	Notify(Severity, NotifyMessage);
}

void UStormSyncNotificationSubsystem::OnPreStartSendingBuffer(const FString& InRemoteAddress, const FString& InRemoteHostname, const int32 InFileCount)
{
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncNotificationSubsystem OnPreStartSendingBuffer InFileCount: %d", InFileCount);
	AddProgressBarNotification(InRemoteAddress, InRemoteHostname);
}

void UStormSyncNotificationSubsystem::OnStartSendingBuffer(const FString& InRemoteAddress, const int32 InBufferSize)
{
	UE_LOGF(LogStormSyncEditor, Display, "UStormSyncNotificationSubsystem OnStartSendingBuffer InBufferSize: %d InRemoteAddress: %ls", InBufferSize, *InRemoteAddress);
	UpdateProgressBarNotificationOnStart(InRemoteAddress, InBufferSize);
}

void UStormSyncNotificationSubsystem::OnReceivingBytes(const FString& InRemoteAddress, const int32 InBytesCount)
{
	UE_LOGF(LogStormSyncEditor, Verbose, "UStormSyncNotificationSubsystem OnReceivingBytes InBytesCount: %d InRemoteAddress: %ls", InBytesCount, *InRemoteAddress);
	UpdateProgressBarNotificationOnIncomingBytes(InRemoteAddress, InBytesCount);
}

void UStormSyncNotificationSubsystem::AddProgressBarNotification(const FString& InRemoteAddress, const FString& InRemoteHostName)
{
	if (Notifications.Contains(InRemoteAddress))
	{
		UE_LOGF(LogStormSyncEditor, Warning, "Trying to add a notification currently active");
		return;
	}

	FStormSyncNotificationTask ProgressTask;

	ProgressTask.Widget = SNew(SStormSyncProgressBarNotification)
		.EndpointAddress(InRemoteAddress)
		.HostName(InRemoteHostName)
		.OnDismissClicked(FSimpleDelegate::CreateUObject(this, &UStormSyncNotificationSubsystem::OnDismissButtonClicked, InRemoteAddress));

	ProgressTask.Widget->SetTotalBytes(0);
	ProgressTask.Widget->SetCurrentBytes(0);

	FNotificationInfo Info(ProgressTask.Widget);
	Info.WidthOverride = 380.f;
	Info.FadeInDuration = 0.2f;
	Info.ExpireDuration = 2.f;
	Info.FadeOutDuration = 1.f;
	Info.bFireAndForget = false;

	ProgressTask.Notification = FSlateNotificationManager::Get().AddNotification(Info);
	ProgressTask.Notification->SetCompletionState(SNotificationItem::CS_Pending);

	Notifications.Add(InRemoteAddress, ProgressTask);
}

void UStormSyncNotificationSubsystem::UpdateProgressBarNotificationOnStart(const FString& InRemoteAddress, const int32 InBufferSize)
{
	if (!Notifications.Contains(InRemoteAddress))
	{
		UE_LOGF(LogStormSyncEditor, Warning, "Trying to update notification for %ls, but it's not active", *InRemoteAddress);
		return;
	}

	const FStormSyncNotificationTask NotificationTask = Notifications.FindChecked(InRemoteAddress);

	if (NotificationTask.Widget.IsValid())
	{
		NotificationTask.Widget->SetCurrentBytes(0);
		NotificationTask.Widget->SetTotalBytes(InBufferSize);
	}
}

void UStormSyncNotificationSubsystem::UpdateProgressBarNotificationOnIncomingBytes(const FString& InRemoteAddress, const int32 InReceivedBytes)
{
	if (!Notifications.Contains(InRemoteAddress))
	{
		UE_LOGF(LogStormSyncEditor, Warning, "Trying to update notification for %ls, but it's not active", *InRemoteAddress);
		return;
	}

	const FStormSyncNotificationTask NotificationTask = Notifications.FindChecked(InRemoteAddress);

	if (NotificationTask.Widget.IsValid())
	{
		NotificationTask.Widget->SetCurrentBytes(InReceivedBytes);

		const float Percent = NotificationTask.Widget->GetPercent();

		if (Percent >= 1.f && NotificationTask.Notification)
		{
			UE_LOGF(LogStormSyncEditor, Display, "Received 100%% bytes, expire and fadeout notif %f", Percent);
			NotificationTask.Notification->ExpireAndFadeout();
			Notifications.Remove(InRemoteAddress);
		}
	}
}

void UStormSyncNotificationSubsystem::OnDismissButtonClicked(const FString InRemoteAddress)
{
	if (!Notifications.Contains(InRemoteAddress))
	{
		UE_LOGF(LogStormSyncEditor, Warning, "Trying to dismiss notification for %ls, but it's not active", *InRemoteAddress);
		return;
	}

	const FStormSyncNotificationTask NotificationTask = Notifications.FindChecked(InRemoteAddress);

	UE_LOGF(LogStormSyncEditor, Verbose, "OnDismissButtonClicked");
	if (NotificationTask.Notification)
	{
		// Expire the notification immediately and ensure it fades quickly so that clicking the buttons feels responsive
		NotificationTask.Notification->SetExpireDuration(0.0f);
		NotificationTask.Notification->SetFadeOutDuration(0.5f);
		NotificationTask.Notification->ExpireAndFadeout();

		Notifications.Remove(InRemoteAddress);
	}
}

#undef LOCTEXT_NAMESPACE
