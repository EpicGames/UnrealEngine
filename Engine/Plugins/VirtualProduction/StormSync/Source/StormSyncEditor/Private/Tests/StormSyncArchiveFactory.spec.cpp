// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/StormSyncArchiveFactory.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"

BEGIN_DEFINE_SPEC(FStormSyncArchiveFactorySpec, "StormSync.StormSyncEditor.StormSyncArchiveFactory", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	UStormSyncArchiveFactory* Factory;

END_DEFINE_SPEC(FStormSyncArchiveFactorySpec)

void FStormSyncArchiveFactorySpec::Define()
{
	BeforeEach([this]()
	{
		// This creates with transient package outer by default
		Factory = NewObject<UStormSyncArchiveFactory>();
		Factory->AddToRoot();
	});

	Describe(TEXT("Sanity check"), [this]()
	{
		It(TEXT("should init member variables as expected"), [this]()
		{
			TestTrue(TEXT("SupportedClass"), Factory->SupportedClass == UStormSyncArchiveData::StaticClass());
			TestEqual(TEXT("Formats"), Factory->Formats, { TEXT("spak;Storm Sync Paked Archive") });
			TestEqual(TEXT("bCreateNew"), Factory->CanCreateNew(), false);
			TestEqual(TEXT("bText"), Factory->bText, false);
			TestEqual(TEXT("bEditorImport"), Factory->bEditorImport, true);
			TestEqual(TEXT("bEditAfterNew"), Factory->bEditAfterNew, false);
		});
	});
	
	Describe(TEXT("FactoryCanImport"), [this]()
	{
		It(TEXT("should import only on .spak files"), [this]()
		{
			TestTrue(TEXT("Foo.spak returns true"), Factory->FactoryCanImport(TEXT("Foo.spak")));
			TestTrue(TEXT(".spak returns true"), Factory->FactoryCanImport(TEXT(".spak")));
			TestTrue(TEXT("Bar/Foo.spak returns true"), Factory->FactoryCanImport(TEXT("Bar/Foo.spak")));
			
			TestFalse(TEXT("Empty filename returns false"), Factory->FactoryCanImport(TEXT("")));
			TestFalse(TEXT("Foo.foo filename returns false"), Factory->FactoryCanImport(TEXT("Foo.foo")));
			TestFalse(TEXT(".foo filename returns false"), Factory->FactoryCanImport(TEXT(".foo")));
			TestFalse(TEXT("Bar/Foo.foo filename returns false"), Factory->FactoryCanImport(TEXT("Bar/Foo.foo")));
		});
	});

	Describe(TEXT("ImportObject"), [this]()
	{
		It(TEXT("should return nullptr on invalid input"), [this]()
		{
			AddExpectedError(TEXT("Can't find file 'invalid file' for import"), EAutomationExpectedErrorFlags::Contains, 1);
			
			bool bCanceled = false;
			const UObject* Result = Factory->ImportObject(nullptr, nullptr, NAME_None, RF_Public | RF_Standalone, TEXT("invalid file"), nullptr, bCanceled);
			TestNull(TEXT("Result is nullptr"), Result);
		});
	});

	Describe(TEXT("FactoryCreateFile"), [this]()
	{
		It(TEXT("Should queue up import next tick but raise error on invalid file"), [this]()
		{
			const FString Filename = TEXT("invalid file");
			
			AddExpectedError(TEXT("Failed to read file 'invalid file' error"), EAutomationExpectedErrorFlags::Contains, 1);
			AddExpectedError(TEXT("Failed to load file 'invalid file' to array"), EAutomationExpectedErrorFlags::Contains, 1);

			UPackage* const Package = GetTransientPackage();

			bool bCanceled = false;
			const UObject* Result = Factory->FactoryCreateFile(UStormSyncArchiveData::StaticClass(), Package, NAME_None, RF_Public | RF_Standalone, Filename, nullptr, GWarn, bCanceled);
			TestNotNull(TEXT("Result is StormSyncArchiveData object"), Result);
			TestTrue(TEXT("Result is StormSyncArchiveData object"), Result->IsA(UStormSyncArchiveData::StaticClass()));
			TestEqual(TEXT("Result.Filename"), Cast<UStormSyncArchiveData>(Result)->Filename, Filename);
		});
	});

	AfterEach([this]()
	{
		if (Factory)
		{
			Factory->RemoveFromRoot();
			
			Factory->MarkAsGarbage();
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			Factory = nullptr;
		}
	});

}
