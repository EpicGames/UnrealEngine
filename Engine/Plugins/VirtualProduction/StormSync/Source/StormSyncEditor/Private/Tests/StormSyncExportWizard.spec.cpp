// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/App.h"
#include "Misc/AutomationTest.h"
#include "StormSyncCoreSettings.h"
#include "Slate/SStormSyncExportWizard.h"

BEGIN_DEFINE_SPEC(FStormSyncExportWizardSpec, "StormSync.StormSyncEditor.SStormSyncExportWizard", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)


END_DEFINE_SPEC(FStormSyncExportWizardSpec)

void FStormSyncExportWizardSpec::Define()
{
	Describe(TEXT("GetDefaultNameFromSelectedPackages"), [this]()
	{
		It(TEXT("should return appropriate default generated name"), [this]()
		{
			const FDateTime Now = FDateTime::Now();

			const UStormSyncCoreSettings* Settings = GetDefault<UStormSyncCoreSettings>();
			check(Settings);
			
			TArray<FName> PackageNames = { TEXT("/Game/AvalancheExamples/A_LiveNews1") };
			FString Result = SStormSyncExportWizard::GetDefaultNameFromSelectedPackages(PackageNames);
			TestEqual(TEXT("One package result"), Result, FString::Printf(TEXT("A_LiveNews1_%s"), *Now.ToString(*Settings->ExportDefaultNameFormatString)));

			PackageNames.Add(TEXT("/Game/AvalancheExamples/A_LiveNews2"));
			Result = SStormSyncExportWizard::GetDefaultNameFromSelectedPackages(PackageNames);
			TestEqual(TEXT("Multi package result"), Result, FString::Printf(TEXT("%s_%s"), FApp::GetProjectName(), *Now.ToString(*Settings->ExportDefaultNameFormatString)));
		});
	});
}
