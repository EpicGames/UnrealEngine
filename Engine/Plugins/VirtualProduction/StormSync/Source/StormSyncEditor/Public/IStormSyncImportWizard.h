// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IStormSyncImportWizard
{
public:
	/** Returns user choice, either import or cancel */
	virtual bool ShouldImport() const = 0;
};
