// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "StormSyncCoreDelegates.h"

class FExtender;
class FMenuBuilder;
class FStormSyncAssetFolderContextMenu;
class IStormSyncImportWizard;
struct FAssetData;
struct FStormSyncImportFileInfo;

/** Main entry point and implementation of StormSync Core Editor module. */
class FStormSyncEditorModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FStormSyncEditorModule& Get()
	{
		static const FName ModuleName = "StormSyncEditor";
		return FModuleManager::LoadModuleChecked<FStormSyncEditorModule>(ModuleName);
	}

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	/** Opens the dialog in a modal new window */
	STORMSYNCEDITOR_API TSharedRef<IStormSyncImportWizard> CreateWizard(const TArray<FStormSyncImportFileInfo>& InFilesToImport, const TArray<FStormSyncImportFileInfo>& InBufferFiles);

	/**
	 * Submenu extension delegate to build submenu for Push and Pull actions (useful for UI extensions)
	 *
	 * This will create a new section and forward entries building to `FStormSyncAssetFolderContextMenu::BuildPushAssetsMenuEntries`
	 *
	 * @param InMenuBuilder Instance of MenuBuilder provided by the top level menu extension
	 * @param InPackageNames List of package names built from the selected assets (FAssetData) menu extensions provides
	 * @param bInIsPushing Whether the delegate should build the submenu for Push (true) or Pull (false) action
	 */
	STORMSYNCEDITOR_API void BuildPushAssetsMenuSection(FMenuBuilder& InMenuBuilder, TArray<FName> InPackageNames, const bool bInIsPushing) const;
	
	/**
	 * Submenu extension delegate to build submenu for compare actions (useful for UI extensions)
	 *
	 * @param InMenuBuilder Instance of MenuBuilder provided by the top level menu extension
	 * @param InPackageNames List of package names built from the selected assets (FAssetData) menu extensions provides
	 */
	STORMSYNCEDITOR_API void BuildCompareWithMenuSection(FMenuBuilder& InMenuBuilder, TArray<FName> InPackageNames) const;
	
	/**
	 * Exposed helper from `FStormSyncAssetFolderContextMenu` utilities to check if a set of PackageNames contains any unsaved (dirty) assets
	 * (useful for UI extensions).
	 *
	 * @param InPackageNames List of package names to check for dirty state
	 * @param OutDisabledReason FText with disabled reason and list of dirty package names displayed as a bullet list,
	 * separated by new lines.
	 * 
	 * @return List of FAssetData with assets in dirty states, empty list otherwise
	 */
	STORMSYNCEDITOR_API TArray<FAssetData> GetDirtyAssets(const TArray<FName>& InPackageNames, FText& OutDisabledReason) const;

	/** Returns a copy of the map of active connections */
	STORMSYNCEDITOR_API TMap<FMessageAddress, FStormSyncConnectedDevice> GetRegisteredConnections();

private:	
	/** Map of active message bus connections over storm sync network */
	TMap<FMessageAddress, FStormSyncConnectedDevice> RegisteredConnections;

	/** Critical section to allow for ThreadSafe updating of registered connections */
	FCriticalSection ConnectionsCriticalSection;
	
	/** References of registered console commands via IConsoleManager */
	TArray<IConsoleObject*> ConsoleCommands;

	/** Holds our context menu handler */
	TSharedPtr<FStormSyncAssetFolderContextMenu> AssetFolderContextMenu;
	
	/** Register Storm Sync Details Customizations */
	void RegisterDetailsCustomizations() const;
	
	/** Unregister Storm Sync Details Customizations */
	void UnregisterDetailsCustomizations() const;

	/** Called from StartupModule and sets up console commands for the plugin via IConsoleManager */
	void RegisterConsoleCommands();

	/** Simple debug helper to log to output console the list of active connections */
	void ExecuteDumpConnections(const TArray<FString>& Args);

	/** Called from ShutdownModule and clears out previously registered console commands */
	void UnregisterConsoleCommands();
	
	/** Command handler for request status command */
	void ExecuteRequestStatusCommand(const TArray<FString>& Args) const;

	/**
	 * Event handler triggered when a new connection on storm sync network is detected.
	 *
	 * The incoming message address ID will be parsed and added to internal RegisteredConnections map if it doesn't exist yet.
	 */
	void OnServiceDiscoveryConnection(const FString& MessageAddressUID, const FStormSyncConnectedDevice& ConnectedDevice);
	
	/**
	 * Event handler when a storm sync connected device changes state (active / inactive).
	 *
	 * This will update the connection bIsValid value in our internal map, which is used to generate context menu entries
	 * enabled / disabled state.
	 */
	void OnServiceDiscoveryStateChange(const FString& MessageAddressUID, EStormSyncConnectedDeviceState State);

	/**
	 * Event handler when remote server status for storm sync connected device has changed (running and listening or stopped).
	 *
	 * This will update the connection bIsServerRunning value in our internal map, which is used to generate context menu entries
	 * enabled / disabled state.
	 */
	void OnServiceDiscoveryServerStatusChange(const FString& MessageAddressUID, bool bIsServerRunning);
	
	/**
	 * Event handler triggered when a connection on storm sync network has been inactive for a certain amount of time (configured via
	 * MessageBusTimeBeforeRemovingInactiveSource in StormSyncTransportSettings).
	 *
	 * At this point, the connection is considered lost and cleaned up from our internal RegisteredConnections map.
	 */
	void OnServiceDiscoveryDisconnection(const FString& MessageAddressUID);

	static FText GetEntryTooltipForRemote(const FMessageAddress& InRemoteAddress, const FStormSyncConnectedDevice& InConnection);
};
