// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "StormSyncNotificationSubsystem.generated.h"

class SStormSyncProgressBarNotification;
struct FStormSyncTransportPullResponse;
struct FStormSyncTransportPushResponse;
struct FStormSyncTransportSyncResponse;

/** Data payload for progress bars to handle an active progress notification */
struct FStormSyncNotificationTask
{
	/** Our notification widget used as content of the notification item */
	TSharedPtr<SStormSyncProgressBarNotification, ESPMode::ThreadSafe> Widget;

	/** Our notification widget */
	TSharedPtr<SNotificationItem, ESPMode::ThreadSafe> Notification;
	
	FStormSyncNotificationTask() = default;
};

/**
 * This subsystem main purpose is to listen for tcp related events triggered from StormSyncTransport layers and module,
 * to provide in-editor integration and handle UI/UX feedback to the end user.
 */
UCLASS(MinimalAPI)
class UStormSyncNotificationSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/** Static convenience method to return storm sync notification subsystem */
	STORMSYNCEDITOR_API static UStormSyncNotificationSubsystem& Get();

	STORMSYNCEDITOR_API TSharedPtr<SNotificationItem> AddSimpleNotification(const FText& InNotificationText);
	STORMSYNCEDITOR_API TSharedPtr<SNotificationItem> AddSimpleNotification(const FNotificationInfo& InInfo);

	void NewPage(const FText& InLabel, const bool bInAppendDateTime = true);
	void Info(const FText& InMessageLogText);
	void Warning(const FText& InMessageLogText);
	void Error(const FText& InMessageLogText);
	void Notify(const EMessageSeverity::Type InSeverity, const FText& InMessageLogText, const bool bInForce = true);

	STORMSYNCEDITOR_API void HandlePushResponse(const TSharedPtr<FStormSyncTransportPushResponse>& InResponse);
	STORMSYNCEDITOR_API void HandlePullResponse(const TSharedPtr<FStormSyncTransportPullResponse>& InResponse);

	void HandleSyncResponse(const TSharedPtr<FStormSyncTransportSyncResponse>& InResponse, const FText& InSuccessTextFormat, const FText& InErrorTextFormat);

protected:
	//~ Begin UEditorSubsystem interface
	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	virtual void Deinitialize() override;
	//~ End UEditorSubsystem interface

	/** Currently active notifications. Key is the remote address. */
	TMap<FString, FStormSyncNotificationTask> Notifications;

	/** Callback handler for FStormSyncCoreDelegates::OnPreStartSendingBuffer delegate */
	void OnPreStartSendingBuffer(const FString& InRemoteAddress, const FString& InRemoteHostname, int32 InFileCount);
	
	/** Callback handler for FStormSyncCoreDelegates::OnStartSendingBuffer delegate */
	void OnStartSendingBuffer(const FString& InRemoteAddress, int32 InBufferSize);
	
	/** Callback handler for FStormSyncCoreDelegates::OnPreStartSendingBuffer delegate */
	void OnReceivingBytes(const FString& InRemoteAddress, int32 InBytesCount);

	/** Called on OnPreStartSendingBuffer(), creates and display a new notification widget, displaying current status of buffer upload. */
	virtual void AddProgressBarNotification(const FString& InRemoteAddress, const FString& InRemoteHostName);

	/**
	 * Called from OnStartSendingBuffer() right before sending the stream. Used to update the total expected size based on buffer to send.
	 */
	virtual void UpdateProgressBarNotificationOnStart(const FString& InRemoteAddress, const int32 InBufferSize);

	/**
	 * Called from OnReceivingBytes() and updates notification widget with received bytes count
	 */
	virtual void UpdateProgressBarNotificationOnIncomingBytes(const FString& InRemoteAddress, const int32 InReceivedBytes);

	/** Dismiss button click handler */
	virtual void OnDismissButtonClicked(const FString InRemoteAddress);

private:
	/** The name of the message output log we will send messages to */
	static constexpr const TCHAR* LogName = TEXT("StormSyncNotifications");

	/** Unique ptr to our message log */
	TUniquePtr<FMessageLog> MessageLog;
};
