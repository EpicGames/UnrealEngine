// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StormSyncEditor : ModuleRules
{
	public StormSyncEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AppFramework",
				"AssetTools",
				"CoreUObject",
				"ContentBrowserData",
				"ContentBrowserAssetDataSource",
				"DesktopPlatform",
				"EditorFramework",
				"EditorSubsystem",
				"Engine",
				"InputCore",
				"MessageLog",
				"RenderCore",
				"Slate",
				"SlateCore",
                "Sockets",
				"StormSyncCore",
				"StormSyncTransportCore",
				"StormSyncTransportClient",
				"StormSyncTransportServer",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
			}
		);
	}
}