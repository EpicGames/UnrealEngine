// Copyright Epic Games, Inc. All Rights Reserved.

#include "Subsystems/StormSyncImportSubsystem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Engine.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/MessageLog.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "RenderingThread.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncCoreSettings.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncImportLog.h"
#include "StormSyncImportTypes.h"
#include "StormSyncTransportSettings.h"
#include "Tasks/IStormSyncImportTask.h"
#include "TimerManager.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Algo/Transform.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "IStormSyncImportWizard.h"
#include "MessageLogModule.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "SourceControlHelpers.h"
#include "StormSyncEditor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/StormSyncNotificationSubsystem.h"
#include "Toolkits/ToolkitManager.h"
#endif

#define LOCTEXT_NAMESPACE "StormSyncImportSubsystem"

void UStormSyncImportSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::Initialize (World: %ls)", *GetNameSafe(GetWorld()));

#if WITH_EDITOR
	// Create a message log for the asset tools to use
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowPages = true;
	MessageLogModule.RegisterLogListing(LogName, LOCTEXT("StormSyncLogLabel", "Storm Sync Editor"), InitOptions);
#endif
}

void UStormSyncImportSubsystem::Deinitialize()
{
	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::Deinitialize (World: %ls)", *GetNameSafe(GetWorld()));
	
#if WITH_EDITOR
	if (FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		// unregister message log
		FMessageLogModule& MessageLogModule = FModuleManager::GetModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing(LogName);
	}
#endif

	// Cleanup delegates
	FStormSyncCoreDelegates::OnRequestImportBuffer.RemoveAll(this);
	FStormSyncCoreDelegates::OnRequestImportFile.RemoveAll(this);
}

UStormSyncImportSubsystem& UStormSyncImportSubsystem::Get()
{
	return *GEngine->GetEngineSubsystem<UStormSyncImportSubsystem>();
}

bool UStormSyncImportSubsystem::EnqueueImportTask(const TSharedPtr<IStormSyncImportSubsystemTask>& InImportTask, const UWorld* InWorld)
{
	if (!InWorld)
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::EnqueueImportTask failed because of invalid world (World: %ls)", *GetNameSafe(InWorld));
		return false;
	}

	// We specifically check for existing pending tasks because of the following scenario:
	//
	// Now that UStormSyncImportWorldSubsystem exists to handle import in -game mode, we need to account for possibly 2 instance of this subsystem at once,
	// especially with PIE (Play in Editor). The currently loaded map in editor viewport will have its own World subsystem listening for imports, and if a PIE
	// session is running (possibly more when simulating clients), each session will have its own World subsystem instance running as well, in addition to the
	// editor one.
	//
	// Only allowing one task at a time may be a bit naive / brute force though

	if (!PendingTasks.IsEmpty())
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::EnqueueImportTask failed because of existing pending tasks (World: %ls)", *GetNameSafe(InWorld));
		return false;
	}

	const bool bIsQueued = PendingTasks.Enqueue(InImportTask);

	if (!NextTickHandler.IsValid())
	{
		NextTickHandler = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UStormSyncImportSubsystem::HandleNextTick));
	}
	
	return bIsQueued;
}

bool UStormSyncImportSubsystem::PerformFileImport(const FString& InFilename)
{
	UE_LOGF(LogStormSyncImport, Display, "FStormSyncEditorUtils::PerformImport for %ls", *InFilename);

	const FStormSyncArchivePtr Reader = FStormSyncArchivePtr(IFileManager::Get().CreateFileReader(*InFilename));
	if (!Reader)
	{
		UE_LOGF(LogStormSyncImport, Error, "Failed to open file '%ls'", *InFilename);
		return false;
	}

	// Dummy package descriptor with name set to imported filename (Note: Should consider serialize package descriptor along buffer header)
	FStormSyncPackageDescriptor PackageDescriptor;
	PackageDescriptor.Name = FPaths::GetBaseFilename(InFilename);

	constexpr bool bShowWizard = true;
	return PerformImport(PackageDescriptor, Reader, bShowWizard);
}

bool UStormSyncImportSubsystem::PerformBufferImport(const FStormSyncPackageDescriptor& InPackageDescriptor, const FStormSyncArchivePtr& InArchive)
{
	if (!InArchive.IsValid())
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::PerformBufferImport failed to import from invalid archive");
		return false;
	}

	const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
	check(Settings);
	
	const bool bTcpDryRun = Settings->IsTcpDryRun();
	const bool bShowWizard = Settings->ShouldShowImportWizard();
	return PerformImport(InPackageDescriptor, InArchive, bShowWizard, bTcpDryRun);
}

bool UStormSyncImportSubsystem::ParsePak(const FStormSyncArchivePtr& InArchive, TArray<FStormSyncImportFileInfo>& OutFileInfos)
{
	if (!InArchive.IsValid())
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::ParsePak failed to import from invalid archive");
		return false;
	}

	if (!InArchive->IsLoading())
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::ParsePak archive \"%ls\" is not loading", *InArchive->GetArchiveName());
		return false;
	}

	FStormSyncCoreExtractArgs ParseArgs;

	ParseArgs.OnGetArchiveForExtract.BindLambda([&OutFileInfos](const FStormSyncFileDependency& InFileDependency, const FString& InDestFilepath) -> FStormSyncArchivePtr
	{
		OutFileInfos.Add(FStormSyncImportFileInfo(InFileDependency, InDestFilepath));
		return nullptr;	// Returning null results in skipping the content of the file in the source archive.
	});

	TArray<FText> Errors;
	if (!FStormSyncCoreUtils::ExtractPakBuffer(*InArchive.Get(), ParseArgs, Errors))
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			UStormSyncNotificationSubsystem::Get().AddSimpleNotification(LOCTEXT("StormSyncParsePak_Error", "Error extracting package. See log for details"));
		}
#endif

		UE_LOGF(LogStormSyncImport, Warning, "FStormSyncEditorUtils::PerformImport - Error extracting package ...");
		for (const FText& Error : Errors)
		{
			UE_LOGF(LogStormSyncImport, Warning, "\t %ls", *Error.ToString());
		}
		return false;
	}
	return true;
}


bool UStormSyncImportSubsystem::PerformImport(const FStormSyncPackageDescriptor& InPackageDescriptor, const FStormSyncArchivePtr& InArchive, bool bShowWizard, bool bDryRun)
{
	if (!InArchive.IsValid())
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::PerformImport failed to import from invalid archive");
		return false;
	}

	if (!InArchive->IsLoading())
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::PerformBufferImport archive \"%ls\" is not loading", *InArchive->GetArchiveName());
		return false;
	}

	// Package descriptor local copy
	FStormSyncPackageDescriptor PackageDescriptor = InPackageDescriptor;
	
	// The list of all files included in the buffer for filename
	TArray<FStormSyncImportFileInfo> AllFileInfos;

	// We have to do a 2 pass parsing so the archive needs to be seekable.
	int64 StartPosition = InArchive->Tell();

	if (StartPosition < 0)
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::PerformBufferImport archive \"%ls\" is not seekable", *InArchive->GetArchiveName());
		return false;
	}

	// Do a first pass to extract all the file information.
	if (!ParsePak(InArchive, AllFileInfos))
	{
		return false;
	}
	
	if (bDryRun)
	{
		UE_LOGF(LogStormSyncImport, Display, "\tOnPakExtract - File Count: %d", AllFileInfos.Num());		
		for (const FStormSyncImportFileInfo& FileInfo : AllFileInfos)
		{
			UE_LOGF(LogStormSyncImport, Display, "\tOnFileExtract");
			UE_LOGF(LogStormSyncImport, Display, "\t\tPackageName: %ls", *FileInfo.FileDependency.PackageName.ToString());
			UE_LOGF(LogStormSyncImport, Display, "\t\tDestFilepath: %ls", *FileInfo.DestFilepath);
			UE_LOGF(LogStormSyncImport, Display, "\t\tFileSize: %lld", FileInfo.FileDependency.FileSize);
		}
		return true;
	}

	// The list of files with changes detected (either size of hash changed)
	TArray<FStormSyncImportFileInfo> FilesToImport;

	{
		FScopedSlowTask SlowTaskHash(AllFileInfos.Num(), LOCTEXT("FiguringOutFileStates", "Figuring out file states..."));
		SlowTaskHash.MakeDialog();
		
		for (const FStormSyncImportFileInfo& FileInfo : AllFileInfos)
		{
			SlowTaskHash.EnterProgressFrame();
			FStormSyncImportFileInfo FileInfoToImport = FileInfo;
			if (UStormSyncImportSubsystem::Get().ShouldFileBeImported(FileInfoToImport))
			{
				FileInfoToImport.bNewAsset = !FPaths::FileExists(FileInfo.DestFilepath);
				FilesToImport.Add(MoveTemp(FileInfoToImport));
			}
		}
	}

	// Important: from this point on, FilesToImport can't be reallocated.
	TArray<const FStormSyncImportFileInfo*> ExistingFilesToImport;
	TArray<const FStormSyncImportFileInfo*> NewFilesToImport;
	TMap<FString, const FStormSyncImportFileInfo*> FilesToImportMap;

	ExistingFilesToImport.Reserve(FilesToImport.Num());
	NewFilesToImport.Reserve(FilesToImport.Num());
	FilesToImportMap.Reserve(FilesToImport.Num());
	
	for (const FStormSyncImportFileInfo& FileInfo : FilesToImport)
	{
		if (FileInfo.bNewAsset)
		{
			NewFilesToImport.Add(&FileInfo);
		}
		else
		{
			ExistingFilesToImport.Add(&FileInfo);
		}
		FilesToImportMap.Add(FileInfo.DestFilepath, &FileInfo);
	}

#if WITH_EDITOR
	if (bShowWizard)
	{
		// Create file import dialog. This is a modal dialog so it will return only after user selection
		const TSharedRef<IStormSyncImportWizard> Wizard = FStormSyncEditorModule::Get().CreateWizard(FilesToImport, AllFileInfos);

		// Early out if user canceled the operation
		if (!Wizard->ShouldImport())
		{
			UStormSyncImportSubsystem::Get().HandlePakPostExtract(PackageDescriptor, 0);
			return true;
		}
	}
#endif

	TArray<FText> ExtractErrors;
	bool bExtractSuccess;
	
	// Begin import process
	{
		FScopedSlowTask SlowTaskExtract(FilesToImport.Num() + ExistingFilesToImport.Num() + NewFilesToImport.Num()
			, LOCTEXT("ImportingFilesToProject", "Importing files to project..."));
		SlowTaskExtract.MakeDialog();

		// Step 1: Prompt for Checkout (for existing assets)
		SlowTaskExtract.EnterProgressFrame(ExistingFilesToImport.Num());
		UStormSyncImportSubsystem::Get().HandleExistingAssetsPreExtract(ExistingFilesToImport, bShowWizard);

		// Step 2: Extract
		FStormSyncCoreExtractArgs ExtractArgs;
		
		ExtractArgs.OnPakPreExtract.BindLambda([&PackageDescriptor](const int32 FileCount)
		{
			UE_LOGF(LogStormSyncImport, Display, "\tOnPakPreExtract - File Count: %d", FileCount);
			UStormSyncImportSubsystem::Get().HandlePakPreExtract(PackageDescriptor, FileCount);
		});
		
		ExtractArgs.OnPakPostExtract.BindLambda([&PackageDescriptor, &FilesToImport](const int32 FileCount)
		{
			// Extract completion
			UStormSyncImportSubsystem::Get().HandlePakPostExtract(PackageDescriptor, FilesToImport.Num());
		});

		ExtractArgs.OnGetArchiveForExtract.BindLambda([&SlowTaskExtract, &FilesToImportMap](const FStormSyncFileDependency& InFileDependency, const FString& InDestFilepath) -> FStormSyncArchivePtr
		{
			if (const FStormSyncImportFileInfo** FoundFileInfo = FilesToImportMap.Find(InDestFilepath))
			{
				SlowTaskExtract.EnterProgressFrame();

				if (UStormSyncImportSubsystem::Get().HandlePakAssetPreExtract(**FoundFileInfo))
				{
					FStormSyncArchivePtr FileWriter(IFileManager::Get().CreateFileWriter(*InDestFilepath));
					if (FileWriter)
					{
						UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::WriteFile - Creating file `%ls`", *InDestFilepath);
					}
					else
					{
						// failed report
						UStormSyncImportSubsystem::Get().HandlePakAssetPostExtract(**FoundFileInfo, /*bFileWritten*/ false);
					}
					return FileWriter;
				}
			}
			else
			{
				UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::PerformImport - FileInfo `%ls` not found in import map.", *InDestFilepath);
			}
			
			// If the files is not different, we don't need to import it nor load it. It will be skipped in the serializer. (not an error)
			return nullptr;
		});

		ExtractArgs.OnArchiveExtracted.BindLambda([&FilesToImportMap](const FStormSyncFileDependency& InFileDependency, const FString& InDestFilepath, const FStormSyncArchivePtr& InArchive)
		{
			// Check that we wrote the expected amount of data.
			const bool bFileWritten = InArchive && InArchive->Tell() == InFileDependency.FileSize;
			if (const FStormSyncImportFileInfo** FoundFileInfo = FilesToImportMap.Find(InDestFilepath))
			{
				UStormSyncImportSubsystem::Get().HandlePakAssetPostExtract(**FoundFileInfo, bFileWritten);
			}
			else
			{
				UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::PerformImport - FileInfo `%ls` not found in import map.", *InDestFilepath);
			}
		});
		
		InArchive->Seek(StartPosition);
		bExtractSuccess = FStormSyncCoreUtils::ExtractPakBuffer(*InArchive.Get(), ExtractArgs, ExtractErrors);
	
		// Step 3: Mark for Add (for new assets)
		SlowTaskExtract.EnterProgressFrame(NewFilesToImport.Num());
		UStormSyncImportSubsystem::Get().HandleNewAssetsPostExtract(NewFilesToImport, bShowWizard);
	}
	
	if (!bExtractSuccess)
	{
		UE_LOGF(LogStormSyncImport, Warning, "FStormSyncEditorUtils::PerformImport - Error extracting package ...");
		for (const FText& Error : ExtractErrors)
		{
			UE_LOGF(LogStormSyncImport, Warning, "\t %ls", *Error.ToString());
		}
	}

	return bExtractSuccess;
}

bool UStormSyncImportSubsystem::HandleNextTick(float InDeltaTime)
{
	if (!PendingTasks.IsEmpty())
	{
		TSharedPtr<IStormSyncImportSubsystemTask> Task;
		while (PendingTasks.Dequeue(Task))
		{
			if (Task.IsValid())
			{
				Task->Run();
			}
		}
	}
	NextTickHandler.Reset();
	return false;
}

bool UStormSyncImportSubsystem::ShouldFileBeImported(FStormSyncImportFileInfo& InFileInfo)
{
	const FStormSyncFileDependency FileDependency = InFileInfo.FileDependency;
	
	const int64 DestFileSize = IFileManager::Get().FileSize(*InFileInfo.DestFilepath);
	// File not existing in local project, add it to files to import and early out
	if (DestFileSize == INDEX_NONE)
	{
		UE_LOGF(LogStormSyncImport, Verbose, "\t\tFile %ls does not exist", *InFileInfo.DestFilepath);
		FString ShortPath = InFileInfo.DestFilepath;
		ShortPath.RemoveFromStart(FPaths::ProjectContentDir());
		InFileInfo.ImportReason = FText::Format(LOCTEXT("FileInfo_MissingFile", "Missing file in local project ({0})"), FText::FromString(ShortPath));
		InFileInfo.ImportReasonTooltip = FText::Format(LOCTEXT("FileInfo_MissingFile_Tooltip", "File {0} does not exist locally"), FText::FromString(InFileInfo.DestFilepath));
		return true;
	}

	UE_LOGF(LogStormSyncImport, Verbose, "\t\tFile %ls exist", *InFileInfo.DestFilepath);

	const bool bSameSize = FileDependency.FileSize == DestFileSize;
	UE_LOGF(LogStormSyncImport, Verbose, "\t\t\tSame Size: %ls (%lld vs %lld)", bSameSize ? TEXT("true") : TEXT("false"), FileDependency.FileSize, DestFileSize);

	// Check file hash from buffer against local file
	const FMD5Hash ExistingFileMD5 = FMD5Hash::HashFile(*InFileInfo.DestFilepath);
	const FString ExistingFileHash = LexToString(ExistingFileMD5);

	const bool bSameHash = FileDependency.FileHash == ExistingFileHash;

	// Mismatched File size or hash
	if (!bSameSize)
	{
		InFileInfo.ImportReason = FText::Format(
			LOCTEXT("FileInfo_MismatchedSize", "Files have different sizes ({0} vs {1})"),
			FText::FromString(FStormSyncCoreUtils::GetHumanReadableByteSize(DestFileSize)),
			FText::FromString(FStormSyncCoreUtils::GetHumanReadableByteSize(FileDependency.FileSize))
		);
		InFileInfo.ImportReasonTooltip = FText::Format(
			LOCTEXT("FileInfo_MismatchedSize_Tooltip", "Mismatched file size ({0} vs {1})"),
			FText::AsNumber(DestFileSize),
			FText::AsNumber(FileDependency.FileSize)
		);
		return true;
	}
	
	if (!bSameHash)
	{
		InFileInfo.ImportReason = LOCTEXT("FileInfo_MismatchedHash", "Files hash are not matching.");
		InFileInfo.ImportReasonTooltip = FText::Format(
			LOCTEXT("FileInfo_MismatchedHash_Tooltip", "Mismatched file hash ({0} vs {1})"),
			FText::FromString(ExistingFileHash),
			FText::FromString(FileDependency.FileHash)
		);
		return true;
	}

	return false;
}

void UStormSyncImportSubsystem::HandlePakPreExtract(const FStormSyncPackageDescriptor& InPackageDescriptor, const int32 FileCount)
{
	UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HandlePakPreExtract - About to extract %d files for %ls", FileCount, *InPackageDescriptor.ToString());

	// Reset closed assets to reopen
	ClosedPackageNames.Empty();

	// Reset extracted file reports
	ExtractedFileReports.Empty();

#if WITH_EDITOR
	// Init a new page for message log to categorize each pak we receive
	FMessageLog MessageLog(LogName);

	const FText PageMessage = FText::Format(LOCTEXT("PakPreExtract_Incoming_Pak", "Handle incoming pak \"{0}\" ({1})"), FText::FromString(InPackageDescriptor.Name), FText::AsDateTime(FDateTime::UtcNow()));
	MessageLog.NewPage(PageMessage);
	MessageLog.Info(FText::FromString(InPackageDescriptor.ToString()));
#endif
}

void UStormSyncImportSubsystem::HandlePakPostExtract(const FStormSyncPackageDescriptor& InPackageDescriptor, const int32 FileCount) const
{
	UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HandlePakPostExtract - Extracted %d files for %ls", FileCount, *InPackageDescriptor.ToString());
	UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HandlePakPostExtract - Should handle reopening of %d assets", ClosedPackageNames.Num());
	
	if (GetDefault<UStormSyncCoreSettings>()->bEnableHotReloadPackages)
	{
		HotReloadPackages(ExtractedFileReports, false);
	}
	
	OpenClosedEditors(ClosedPackageNames);

	for (const FStormSyncEditorFileReport& FileReport : ExtractedFileReports)
	{
		if (FileReport.bSuccess)
		{
			FStormSyncCoreDelegates::OnPakAssetExtracted.Broadcast(FileReport.PackageName, FileReport.DestFilepath);
		}
	}
	
#if WITH_EDITOR
	// Determine log severity and message based on our stored reports
	EMessageSeverity::Type Severity = EMessageSeverity::Info;
	FText LogMessage = FText::Format(LOCTEXT("PakPostExtract_Extract_Success", "Extracted {0} files successfully."), FText::AsNumber(FileCount));

	const TArray<FStormSyncEditorFileReport> ErrorFileReports = ExtractedFileReports.FilterByPredicate([](const FStormSyncEditorFileReport& Report)
	{
		return Report.bSuccess == false;
	});

	if (!ErrorFileReports.IsEmpty())
	{
		Severity = EMessageSeverity::Error;
		LogMessage = FText::Format(LOCTEXT("PakPostExtract_Extract_Failed", "Extracted {0} files. Some content could not be extracted."), FText::AsNumber(FileCount));
	}

	const FText LogHeading = FText::Format(LOCTEXT("PakPostExtract_Heading", "Received content spak \"{0}\"."), FText::FromString(InPackageDescriptor.Name));
	const FText NotifyMessage = FText::Format(LOCTEXT("PakPostExtract_Message_Notification", "{0}\n{1}"), LogHeading, LogMessage);
	
	FMessageLog MessageLog(LogName);
	MessageLog.Notify(NotifyMessage, Severity, true);
#endif
}

void UStormSyncImportSubsystem::HandleExistingAssetsPreExtract(TConstArrayView<const FStormSyncImportFileInfo*> InExistingFiles, bool bInShowPrompt)
{
#if WITH_EDITOR
	UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HandleExistingAssets - Package Count: %d", InExistingFiles.Num());

	TArray<UPackage*> ExistingPackages;

	// Worst case: all files already exist in destination
	ExistingPackages.Reserve(InExistingFiles.Num());

	for (const FStormSyncImportFileInfo* FileInfo : InExistingFiles)
	{
		const FString PackageName = FileInfo->FileDependency.PackageName.ToString();

		if (UPackage* const Package = LoadPackage(nullptr, *PackageName, LOAD_None))
		{
			UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::HandleExistingAssets - Existing Package found for: %ls", *PackageName);
			ExistingPackages.Add(Package);
		}
		else
		{
			UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::HandleExistingAssets - Package not found for: %ls. Not an Existing Asset", *PackageName);
		}
	}

	bool bSucceeded;

	TArray<UPackage*> PackagesCheckedOut;
	TArray<UPackage*> PackagesNotNeedingCheckout;

	if (bInShowPrompt)
	{
		constexpr bool bCheckDirty = false;
		constexpr bool bAllowSkip = false;
		constexpr bool bPromptingAfterModify = false;

		bSucceeded = FEditorFileUtils::PromptToCheckoutPackages(bCheckDirty
			, ExistingPackages
			, &PackagesCheckedOut
			, &PackagesNotNeedingCheckout
			, bPromptingAfterModify
			, bAllowSkip);
	}
	else
	{
		constexpr bool bErrorIfAlreadyCheckedOut = false;
		constexpr bool bConfirmPackageBranchCheckOutStatus = false;

		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		if (SourceControlProvider.IsEnabled())
		{
			UE_LOGF(LogStormSyncImport, Log, "UStormSyncImportSubsystem::HandleExistingAssets - Checking out %d packages with %ls"
				, ExistingPackages.Num()
				, *SourceControlProvider.GetName().ToString());

			bSucceeded = ECommandResult::Succeeded == FEditorFileUtils::CheckoutPackages(ExistingPackages
				, &PackagesCheckedOut
				, bErrorIfAlreadyCheckedOut
				, bConfirmPackageBranchCheckOutStatus);
		}
		else
		{
			UE_LOGF(LogStormSyncImport, Log, "UStormSyncImportSubsystem::HandleExistingAssets - No Source Control found. Making %d packages writeable"
				, ExistingPackages.Num());

			bSucceeded = UStormSyncImportSubsystem::MakePackagesWriteable(ExistingPackages) > 0;
		}
	}

	if (!bSucceeded)
	{
		// TODO: at this point in time, the whole import could be cancelled and failed
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::HandleExistingAssets - CheckoutPackages failed");
	}
#endif
}

bool UStormSyncImportSubsystem::HandleNewAssetsPostExtract(TConstArrayView<const FStormSyncImportFileInfo*> InNewFiles, bool bInShowPrompt)
{
#if WITH_EDITOR
	if (!GEditor || GIsAutomationTesting || IsRunningCommandlet())
	{
		return false;
	}

	if (!ISourceControlModule::Get().IsEnabled())
	{
		UE_LOGF(LogStormSyncImport, Log, "UStormSyncImportSubsystem::HandleNewAssets - No Source Control found");
		return false;
	}

	if (!GetDefault<UEditorLoadingSavingSettings>()->bSCCAutoAddNewFiles)
	{
		UE_LOGF(LogStormSyncImport, Log, "UStormSyncImportSubsystem::HandleNewAssets - Source Control Auto Add New Files is disabled.");
		return false;
	}

	if (!ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		UE_LOGF(LogStormSyncImport, Log, "UStormSyncImportSubsystem::HandleNewAssets - Source Control Provider is not available.");
		return false;
	}

	TArray<FString> FilesToAutoAdd;
	FilesToAutoAdd.Reserve(InNewFiles.Num());

	Algo::Transform(InNewFiles, FilesToAutoAdd,
		[](const FStormSyncImportFileInfo* InFileInfo)
		{
			return InFileInfo->DestFilepath;
		});

	UE_LOGF(LogStormSyncImport, Log, "UStormSyncImportSubsystem::HandleNewAssets - Found %d files to auto-add", FilesToAutoAdd.Num());

	const bool bSilent = !bInShowPrompt;
	return USourceControlHelpers::CheckOutOrAddFiles(FilesToAutoAdd, bSilent);
#else
	return false;
#endif
}

bool UStormSyncImportSubsystem::HandlePakAssetPreExtract(const FStormSyncImportFileInfo& InFileInfo)
{
	const FStormSyncFileDependency& FileDependency = InFileInfo.FileDependency;
	const FString& DestFilepath = InFileInfo.DestFilepath;
	
	UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HandlePakAssetPreExtract - Handle extracted package: %ls", *FileDependency.ToString());
	UE_LOGF(LogStormSyncImport, Display, "\tshould extract to %ls", *DestFilepath);

	FStormSyncEditorFileReport FileReport;
	FileReport.PackageName = FileDependency.PackageName;
	FileReport.DestFilepath = DestFilepath;

	if (InFileInfo.bNewAsset)
	{
		return true;	// Nothing to do.
	}
	
	// If File exists locally, we should:
	//
	// 1. Close Editors
	// 2. Delete file -- not necessary if using hot reload.
	// 3. Write/Overwrite new file version from incoming pak
	// 4. Hot Reload packages
	// 5. Reopen closed editor

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Close the asset editors if opened, we will attempt to reopen when we receive PostExtract event (handled in OnPakPostExtract)
	TArray<FAssetData> AssetsToDelete;
	AssetRegistryModule.Get().GetAssetsByPackageName(FileDependency.PackageName, AssetsToDelete);

	TArray<FAssetData> ClosedAssets;
	CloseEditors(AssetsToDelete, ClosedAssets);

	// Store closed assets to reopen
	for (FAssetData& AssetData : ClosedAssets)
	{
		ClosedPackageNames.AddUnique(AssetData.PackageName.ToString());
	}

	// Deleting the assets shouldn't be necessary when hot reload is enabled.
	// TODO: We might still need to identify the assets that need to be deleted.
	// See ConcertSyncClientUtil::PurgePackages.
	if (!GetDefault<UStormSyncCoreSettings>()->bEnableHotReloadPackages)
	{
		if (!DeleteAssets(AssetsToDelete))
		{
			UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::HandlePakAssetPreExtract - Unable to unload asset for package: %ls", *FileDependency.PackageName.ToString());
			FileReport.bSuccess = false;
			ExtractedFileReports.Add(FileReport);
			return false;
		}
	}
	else
	{
		// They do this in FConcertClientPackageManager::SavePackageFile prior to writing data to file.
		FlushPackageLoading(FileDependency.PackageName.ToString());
	}

	return true;
}

void UStormSyncImportSubsystem::HandlePakAssetPostExtract(const FStormSyncImportFileInfo& InFileInfo, bool bInFileWritten)
{
	const FStormSyncFileDependency& FileDependency = InFileInfo.FileDependency;
	const FString& DestFilepath = InFileInfo.DestFilepath;

	FStormSyncEditorFileReport FileReport;
	FileReport.PackageName = FileDependency.PackageName;
	FileReport.DestFilepath = DestFilepath;
	FileReport.bSuccess = bInFileWritten;

	if (FileReport.bSuccess)
	{
		UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HandlePakAssetPostExtract - Package: \"%ls\" was extracted to \"%ls\"", *FileDependency.ToString(), *DestFilepath);
	}
	else
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::HandlePakAssetPostExtract - Package: \"%ls\" failed to extract to \"%ls\"", *FileDependency.ToString(), *DestFilepath);
	}

#if WITH_EDITOR
	FMessageLog MessageLog(LogName);
	if (FileReport.bSuccess)
	{
		MessageLog.Info(FText::Format(LOCTEXT("PakAssetExtract_SucessfulExtract", "Extracted {0} file to {1}."), FText::FromName(FileDependency.PackageName), FText::FromString(DestFilepath)));
	}
	else
	{
		MessageLog.Error(FText::Format(LOCTEXT("PakAssetExtract_FileWrite_Failed", "Failed to extract {0} file to {1}"), FText::FromName(FileDependency.PackageName), FText::FromString(DestFilepath)));
	}
#endif

	ExtractedFileReports.Add(MoveTemp(FileReport));
}

void UStormSyncImportSubsystem::CloseEditors(const TArray<FAssetData>& InAssets, TArray<FAssetData>& OutClosedAssets)
{
#if WITH_EDITOR
	for (const FAssetData& AssetData : InAssets)
	{
		const UObject* Asset = AssetData.FastGetAsset();
		UE_LOGF(LogStormSyncImport, Verbose, "\tClosing asset: %ls (UObject: %ls)", *AssetData.GetFullName(), *GetNameSafe(Asset));

		if (!Asset)
		{
			continue;
		}

		const TSharedPtr<IToolkit> AssetEditor = FToolkitManager::Get().FindEditorForAsset(Asset);
		if (AssetEditor.IsValid() && IsAssetCurrentlyBeingEdited(AssetEditor, Asset))
		{
			FToolkitManager::Get().CloseToolkit(AssetEditor.ToSharedRef());

			// Store assets to delete and reopen
			OutClosedAssets.Add(AssetData);
		}
	}
#endif
}

void UStormSyncImportSubsystem::OpenClosedEditors(const TArray<FString>& ClosedPackageNames)
{
	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::OpenClosedEditors - ClosedPackageNames: %d", ClosedPackageNames.Num());
#if WITH_EDITOR
	if (!GEditor)
	{
		return;
	}
	
	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::OpenClosedEditors - ClosedPackageNames: %d (WITH_EDITOR)", ClosedPackageNames.Num());
	TArray<UObject*> ObjectsToReopen;
	for (FString ClosedPackageName : ClosedPackageNames)
	{
		if (UObject* Object = StaticLoadObject(UObject::StaticClass(), nullptr, *ClosedPackageName))
		{
			ObjectsToReopen.Add(Object);
		}
	}

	if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		AssetEditorSubsystem->OpenEditorForAssets(ObjectsToReopen);
	}
#endif
}

bool UStormSyncImportSubsystem::DeleteAssets(const TArray<FAssetData>& AssetsToDelete, const bool bShowConfirmation)
{
#if WITH_EDITOR
	TArray<TWeakObjectPtr<UPackage>> PackageFilesToDelete;
	TArray<UObject*> ObjectsToDelete;

	for (const FAssetData& AssetData : AssetsToDelete)
	{
		UObject* ObjectToDelete = AssetData.GetAsset({ULevel::LoadAllExternalObjectsTag});
		// Assets can be loaded even when their underlying type/class no longer exists...
		if (ObjectToDelete != nullptr)
		{
			ObjectsToDelete.Add(ObjectToDelete);
		}
		else if (AssetData.IsUAsset())
		{
			// ... In this cases there is no underlying asset or type so remove the package itself directly after confirming it's valid to do so.
			FString PackageFilename;
			if (!FPackageName::DoesPackageExist(AssetData.PackageName.ToString(), &PackageFilename))
			{
				// Could not determine filename for package so we can not delete
				ensureMsgf(false, TEXT("Could not determine filename for package %s so we can not delete"), *AssetData.PackageName.ToString());
				continue;
			}

			if (UPackage* Package = FindPackage(nullptr, *AssetData.PackageName.ToString()))
			{
				PackageFilesToDelete.Add(Package);
			}
		}
	}

	int32 NumObjectsToDelete = ObjectsToDelete.Num();
	if (NumObjectsToDelete > 0)
	{
		// First try with regular delete objects
		NumObjectsToDelete = ObjectTools::DeleteObjects(ObjectsToDelete, bShowConfirmation);
		if (NumObjectsToDelete != ObjectsToDelete.Num())
		{
			// If delete objects failed, then fallback to force delete
			NumObjectsToDelete = ObjectTools::ForceDeleteObjects(ObjectsToDelete, bShowConfirmation);
		}
	}

	const int32 NumPackagesToDelete = PackageFilesToDelete.Num();
	if (NumPackagesToDelete > 0)
	{
		TArray<UPackage*> PackagePointers;
		for (const TWeakObjectPtr<UPackage>& PkgIt : PackageFilesToDelete)
		{
			if (UPackage* Package = PkgIt.Get())
			{
				PackagePointers.Add(Package);
			}
		}

		if (PackagePointers.Num() > 0)
		{
			constexpr bool bPerformReferenceCheck = true;
			ObjectTools::CleanupAfterSuccessfulDelete(PackagePointers, bPerformReferenceCheck);
		}
	}

	const int32 TotalDeletedObjects = NumPackagesToDelete + NumObjectsToDelete;
	if (TotalDeletedObjects != AssetsToDelete.Num())
	{
		UE_LOGF(LogStormSyncImport, Warning, "Failed to delete assets (Deleted %d assets while we were expecting to delete %d assets)", TotalDeletedObjects, AssetsToDelete.Num());
		return false;
	}
#endif

	return true;
}

int32 UStormSyncImportSubsystem::MakePackagesWriteable(TConstArrayView<UPackage*> InPackages)
{
	int32 PackagesMadeWriteable = 0;

	for (UPackage* Package : InPackages)
	{
		if (!Package)
		{
			continue;
		}

		FString Filename;
		if (!FPackageName::DoesPackageExist(Package->GetName(), &Filename))
		{
			continue;
		}

		// Remove the read only flag from the current file attributes
		if (FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*Filename, false))
		{
			++PackagesMadeWriteable;
		}
	}

	return PackagesMadeWriteable;
}

bool UStormSyncImportSubsystem::IsAssetCurrentlyBeingEdited(const TSharedPtr<IToolkit>& InAssetEditor, const UObject* InAsset)
{
#if WITH_EDITOR
	if (!InAsset)
	{
		return false;
	}

	if (InAssetEditor.IsValid() && InAssetEditor->IsAssetEditor())
	{
		if (const TArray<UObject*>* EditedObjects = InAssetEditor->GetObjectsCurrentlyBeingEdited())
		{
			for (const UObject* EditedObject : *EditedObjects)
			{
				if (EditedObject == InAsset)
				{
					return true;
				}
			}
		}
	}
#endif

	return false;
}

bool UStormSyncImportSubsystem::WriteFile(const FString& DestFilepath, const uint64& FileSize, uint8* FileBuffer)
{
	TUniquePtr<FArchive> AssetHandle(IFileManager::Get().CreateFileWriter(*DestFilepath));
	if (!AssetHandle.IsValid())
	{
		return false;
	}

	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportSubsystem::WriteFile - Creating file `%ls`", *DestFilepath);

	// Write to asset
	AssetHandle->Serialize(FileBuffer, FileSize);

	// Close the file
	AssetHandle->Close();
	AssetHandle.Reset();

	return true;
}

// Copied from FlushPackageLoading in ConcertSyncClientUtil.cpp.
void UStormSyncImportSubsystem::FlushPackageLoading(const FString& InPackageName, bool bInForceBulkDataLoad)
{	
	if (UPackage* ExistingPackage = FindPackage(nullptr, *InPackageName))
	{
		if (!ExistingPackage->IsFullyLoaded())
		{
			FlushAsyncLoading();
			ExistingPackage->FullyLoad();
		}

		if (bInForceBulkDataLoad)
		{
			ResetLoaders(ExistingPackage);
		}
		else if (ExistingPackage->GetLinker())
		{
			ExistingPackage->GetLinker()->Detach();
		}
	}
}


// Copied from HotReloadPackages in ConcertSyncClientUtil.cpp.
void UStormSyncImportSubsystem::HotReloadPackages(const TArray<FStormSyncEditorFileReport>& InExtractedFileReports, bool bInInteractiveHotReload)
{
	TArray<FName> PackageNames;
	PackageNames.Reserve(InExtractedFileReports.Num());
	for (const FStormSyncEditorFileReport& FileReport : InExtractedFileReports)
	{
		if (FileReport.bSuccess)
		{
			PackageNames.Add(FileReport.PackageName);
		}
	}
	
	if (PackageNames.IsEmpty())
	{
		return;
	}

	UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HotReloadPackages - Reloading %d packages.", PackageNames.Num());
	
	// Flush loading and clean-up any temporary placeholder packages (due to a package previously being missing on disk)
	FlushAsyncLoading();
	{
		bool bRunGC = false;
		for (const FName& PackageName : PackageNames)
		{
			bRunGC |= FLinkerLoad::RemoveKnownMissingPackage(PackageName);
		}
		if (bRunGC)
		{
			UE_LOGF(LogStormSyncImport, Display, "UStormSyncImportSubsystem::HotReloadPackages - Garbage Collecting...");
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}
	
	// Find the packages in-memory to content hot-reload
	TArray<UPackage*> ExistingPackages;
	ExistingPackages.Reserve(PackageNames.Num());

	for (const FName& PackageName : PackageNames)
	{
		if (UPackage* ExistingPackage = FindPackage(nullptr, *PackageName.ToString()))
		{
			if (ExistingPackage->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				ExistingPackage->ClearPackageFlags(PKG_NewlyCreated);
			}
			ExistingPackages.Add(ExistingPackage);
		}
	}

	if (!ExistingPackages.IsEmpty())
	{
		FlushRenderingCommands();

		FText ErrorMessage;
#if WITH_EDITOR
		const UPackageTools::EReloadPackagesInteractionMode InteractionMode = bInInteractiveHotReload ?
			UPackageTools::EReloadPackagesInteractionMode::Interactive :
			UPackageTools::EReloadPackagesInteractionMode::AssumePositive;

		UPackageTools::ReloadPackages(ExistingPackages, ErrorMessage,InteractionMode);
#endif

		if (!ErrorMessage.IsEmpty())
		{
			UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::HotReloadPackages: %ls", *ErrorMessage.ToString());
		}
	}
}

#undef LOCTEXT_NAMESPACE
