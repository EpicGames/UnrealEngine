// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "StormSyncCommonTypes.h"
#include "StormSyncPackageDescriptor.h"
#include "Subsystems/EngineSubsystem.h"
#include "StormSyncImportSubsystem.generated.h"

class IStormSyncImportSubsystemTask;
class IToolkit;
struct FStormSyncImportFileInfo;

/** Holds information about extraction process for a given file */
struct FStormSyncEditorFileReport
{
	/** Whether the extraction was successful or not */
	bool bSuccess = true;
	
	/** Original package name */
	FName PackageName;
	
	/** Fully qualified path we tried to extract to */
	FString DestFilepath;

	FStormSyncEditorFileReport() = default;
};

/**
 * UStormSyncImportSubsystem
 * 
 * Subsystem for importing assets in the editor extracted from a local exported storm sync buffer, or coming from a network request.
 *
 * Contains utility functions and callbacks for hooking into importing.
 */
UCLASS()
class UStormSyncImportSubsystem final : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin USubsystem interface
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem interface

	/** Static Convenience method to return storm sync import subsystem */
	static UStormSyncImportSubsystem& Get();

	/**
	 * Enqueue an import task (and handle on next tick) only if no pending existing task already exist
	 *
	 * @param InImportTask Task to enqueue
	 * @param InWorld World to get the timer manager from
	 * @return true if the task was added, false otherwise. 
	 */
	bool EnqueueImportTask(const TSharedPtr<IStormSyncImportSubsystemTask>& InImportTask, const UWorld* InWorld);

	/**
	 * Takes an absolute filename (a storm sync pak) and begins extraction process
	 *
	 * Comparison between file coming from buffer and current file state in local file system is done to figure out
	 * the list of files that needs to be extracted and updated in this editor instance.
	 *
	 * @param InFilename Absolute path name for the storm sync archive to import
	 * 
	 * @return Operation status, true if successful
	 */
	static bool PerformFileImport(const FString& InFilename);

	/**
	 * Takes a raw buffer (a storm sync pak) and begins extraction process
	 *
	 * Comparison between file coming from buffer and current file state in local file system is done to figure out
	 * the list of files that needs to be extracted and updated in this editor instance.
	 *
	 * @param InPackageDescriptor The package descriptor for this buffer extraction. Only used for UI message logs.
	 * @param InBuffer Raw buffer we'd like to extract files from to local project
	 * 
	 * @return Operation status, true if successful
	 */
	static bool PerformBufferImport(const FStormSyncPackageDescriptor& InPackageDescriptor, const FStormSyncArchivePtr& InArchive);

	/**
	 * Takes a raw buffer (a storm sync pak) and begins extraction process
	 *
	 * Comparison between file coming from buffer and current file state in local file system is done to figure out
	 * the list of files that needs to be extracted and updated in this editor instance.
	 *
	 * @param InPackageDescriptor The package descriptor for this buffer extraction. Only used for UI message logs.
	 * @param InArchive Source archive to be parsed to extract content
	 * @param bShowWizard Whether or not to show the import wizard
	 * @param bDryRun Whether we should proceed to extraction or simply log to console buffer content
	 * 
	 * @return Operation status, true if successful
	 */
	static bool PerformImport(const FStormSyncPackageDescriptor& InPackageDescriptor, const FStormSyncArchivePtr& InArchive, bool bShowWizard = false, bool bDryRun = false);

private:
	/**
	 * Parse the buffer to extract all the file information (but skip content)
	 * @param InArchive Source archive of the pak
	 * @param OutFileInfos File info in the pak.
	 * @return true if parsing was successful.
	 */
	static bool ParsePak(const FStormSyncArchivePtr& InArchive, TArray<FStormSyncImportFileInfo>& OutFileInfos);
	
private:
	/** The name of the message output log we will send messages to */
	static constexpr const TCHAR* LogName = TEXT("StormSyncImport");

	FTSTicker::FDelegateHandle NextTickHandler;

	/* Tasks waiting to be run next tick */
	TQueue<TSharedPtr<IStormSyncImportSubsystemTask>> PendingTasks;
	
	/** Store closed assets to reopen during extraction process */
	TArray<FString> ClosedPackageNames;
	
	/** Store extraction process data */
	TArray<FStormSyncEditorFileReport> ExtractedFileReports;

	/* Run deferred logic waiting to be run next tick */
	bool HandleNextTick(float InDeltaTime);
	
	/**
	 * Checks local project file for diff against provided file info and dest path.
	 *
	 * If a change is detected (either missing locally or mismatched file size / hash), OutFilesToImport is filled with a new file info indicating that the
	 * file needs an import.
	 */
	static bool ShouldFileBeImported(FStormSyncImportFileInfo& InFileInfo);

	/** Event handler for core delegate when a pak extraction starts */
	void HandlePakPreExtract(const FStormSyncPackageDescriptor& InPackageDescriptor, int32 FileCount);

	/** Event handler for core delegate when a pak extraction is ended */
	void HandlePakPostExtract(const FStormSyncPackageDescriptor& InPackageDescriptor, int32 FileCount) const;

	/** Event handler for existing asset checkout prior to extracting the new overwriting assets */
	void HandleExistingAssetsPreExtract(TConstArrayView<const FStormSyncImportFileInfo*> InExistingFiles, bool bInShowPrompt);

	/** Event handler for auto-adding new assets post extracting them */
	bool HandleNewAssetsPostExtract(TConstArrayView<const FStormSyncImportFileInfo*> InNewFiles, bool bInShowPrompt);

	/** Event handler for when a file is about to be extracted from an incoming pak */
	bool HandlePakAssetPreExtract(const FStormSyncImportFileInfo& InFileInfo);

	/** Event handler for when a file has been extracted from an incoming pak */
	void HandlePakAssetPostExtract(const FStormSyncImportFileInfo& InFileInfo, bool bInFileWritten);

	/** Checks if given asset is currently being edited, and if so, closes the editor (required prior to file deletion) */
	static void CloseEditors(const TArray<FAssetData>& InAssets, TArray<FAssetData>& OutClosedAssets);
	
	/** Takes a list of package names to ask the Asset Editor subsystem to open */
	static void OpenClosedEditors(const TArray<FString>& ClosedPackageNames);

	/**
	 * Our own custom version of ObjectTools::DeleteAssets
	 *
	 * We want to force delete the file, to allow assets with on disk references (for things like Textures in a Motion Design asset) to be properly overwritten,
	 * if referenced in memory, opened in editor etc.
	 *
	 * ObjectTools::DeleteAssets will only succeed if the AssetDeleteModel doesn't have on disk references (see CanDelete / CanForceDelete)
	 */
	static bool DeleteAssets(const TArray<FAssetData>& AssetsToDelete, bool bShowConfirmation = false);

	/** Makes the given packages writeable. Should only be used if Source Control is disabled. Returns number of packages that succeeded */
	static int32 MakePackagesWriteable(TConstArrayView<UPackage*> InPackages);

	/** Checks if given UObject asset is currently being edited by an Asset Editor (is it opened in editor right now?) */
	static bool IsAssetCurrentlyBeingEdited(const TSharedPtr<IToolkit>& InAssetEditor, const UObject* InAsset);

	/** Write file buffer to destination filepath */
	static bool WriteFile(const FString& DestFilepath, const uint64& FileSize, uint8* FileBuffer);

	/** Blocks till all pending package/ linker requests are fulfilled. */
	static void FlushPackageLoading(const FString& InPackageName, bool bInForceBulkDataLoad = true);

	/** Attempts to reload the specified top-level packages. */
	static void HotReloadPackages(const TArray<FStormSyncEditorFileReport>& InExtractedFileReports, bool bInInteractiveHotReload);
};
