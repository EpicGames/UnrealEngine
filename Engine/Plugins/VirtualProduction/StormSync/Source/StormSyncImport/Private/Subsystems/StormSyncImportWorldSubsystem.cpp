// Copyright Epic Games, Inc. All Rights Reserved.


#include "Subsystems/StormSyncImportWorldSubsystem.h"

#include "Engine/World.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncImportLog.h"
#include "Subsystems/StormSyncImportSubsystem.h"
#include "Tasks/StormSyncImportBufferTask.h"
#include "Tasks/StormSyncImportFilesTask.h"

void UStormSyncImportWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportWorldSubsystem::Initialize (World: %ls) - %ls", *GetNameSafe(GetWorld()), *GetName());

	FStormSyncCoreDelegates::OnRequestImportFile.AddUObject(this, &UStormSyncImportWorldSubsystem::HandleImportFile);
	FStormSyncCoreDelegates::OnRequestImportBuffer.AddUObject(this, &UStormSyncImportWorldSubsystem::HandleImportBuffer);
}

void UStormSyncImportWorldSubsystem::Deinitialize()
{
	Super::Deinitialize();
	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportWorldSubsystem::Deinitialize (World: %ls) - %ls", *GetNameSafe(GetWorld()), *GetName());

	FStormSyncCoreDelegates::OnRequestImportFile.RemoveAll(this);
	FStormSyncCoreDelegates::OnRequestImportBuffer.RemoveAll(this);
}

void UStormSyncImportWorldSubsystem::HandleImportFile(const FString& InFilename) const
{
	UE_LOGF(LogStormSyncImport, Verbose, "UStormSyncImportWorldSubsystem::HandleImportFile InFilename: %ls (World: %ls)", *InFilename, *GetNameSafe(GetWorld()));
	const UWorld* World = GetWorld();
	if (!World)
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportWorldSubsystem::ImportNextTick failed because of invalid world (World: %ls)", *GetNameSafe(GetWorld()));
		return;
	}

	UStormSyncImportSubsystem::Get().EnqueueImportTask(MakeShared<FStormSyncImportFilesTask>(InFilename), GetWorld());
}

void UStormSyncImportWorldSubsystem::HandleImportBuffer(const FStormSyncPackageDescriptor& InPackageDescriptor, const FStormSyncArchivePtr& InArchive) const
{
	if (!InArchive.IsValid())
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportSubsystem::HandleImportBuffer failed from invalid archive");
		return;
	}
	
	UE_LOGF(LogStormSyncImport, 
		Verbose,
		"UStormSyncImportWorldSubsystem::HandleImportBuffer InPackageDescriptor: %ls, BufferSize: %lld (World: %ls)",
		*InPackageDescriptor.ToString(),
		InArchive->TotalSize(),
		*GetNameSafe(GetWorld())
	);

	const UWorld* World = GetWorld();
	if (!World)
	{
		UE_LOGF(LogStormSyncImport, Error, "UStormSyncImportWorldSubsystem::HandleImportBuffer failed because of invalid world (World: %ls)", *GetNameSafe(GetWorld()));
		return;
	}

	UStormSyncImportSubsystem::Get().EnqueueImportTask(MakeShared<FStormSyncImportBufferTask>(InPackageDescriptor, InArchive), GetWorld());
}
