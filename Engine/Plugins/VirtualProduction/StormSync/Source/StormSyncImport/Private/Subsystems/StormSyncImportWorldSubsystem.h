// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StormSyncCommonTypes.h"
#include "StormSyncPackageDescriptor.h"
#include "Subsystems/WorldSubsystem.h"
#include "StormSyncImportWorldSubsystem.generated.h"

/**
 * UStormSyncImportWorldSubsystem
 * 
 * **World** Subsystem for importing assets in -game mode, from a local exported storm sync buffer, or coming from a network request.
 *
 * Bulk of the import logic is handled by UStormSyncImportSubsystem, this world subsystem registers a delegate on FStormSyncCoreDelegates
 * OnRequestImportFile / 
 */

UCLASS()
class UStormSyncImportWorldSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	
	void HandleImportFile(const FString& InFilename) const;
	void HandleImportBuffer(const FStormSyncPackageDescriptor& InPackageDescriptor, const FStormSyncArchivePtr& InArchive) const;
};
