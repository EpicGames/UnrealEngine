// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Interface for tasks that need delayed execution */
class IStormSyncImportSubsystemTask
{
public:
	virtual ~IStormSyncImportSubsystemTask() = default;

	virtual void Run() = 0;
};
