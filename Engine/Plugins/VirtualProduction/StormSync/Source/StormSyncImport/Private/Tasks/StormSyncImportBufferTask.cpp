// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/StormSyncImportBufferTask.h"

#include "StormSyncImportLog.h"
#include "Subsystems/StormSyncImportSubsystem.h"

void FStormSyncImportBufferTask::Run()
{
	if (!Archive.IsValid())
	{
		UE_LOGF(LogStormSyncImport, Error, "FStormSyncImportBufferTask::Run failed on invalid archive");
		return;
	}

	UE_LOGF(LogStormSyncImport, Display, "FStormSyncImportBufferTask::Run for buffer of size %lld", Archive->TotalSize());
	UStormSyncImportSubsystem::Get().PerformBufferImport(PackageDescriptor, MoveTemp(Archive));
}
