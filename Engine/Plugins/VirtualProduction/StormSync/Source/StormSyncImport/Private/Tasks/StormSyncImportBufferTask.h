// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StormSyncCommonTypes.h"
#include "StormSyncPackageDescriptor.h"
#include "Tasks/IStormSyncImportTask.h"

/** Import files from Buffer implementation for tasks that need delayed execution */
class FStormSyncImportBufferTask final : public IStormSyncImportSubsystemTask
{
public:
	explicit FStormSyncImportBufferTask(const FStormSyncPackageDescriptor& InPackageDescriptor, const FStormSyncArchivePtr& InArchive)
		: PackageDescriptor(InPackageDescriptor)
		, Archive(InArchive)
	{
	}

	virtual void Run() override;

private:
	/** Metadata info about buffer being extracted */
	FStormSyncPackageDescriptor PackageDescriptor;
	
	/** Archive to import */
	FStormSyncArchivePtr Archive;
};