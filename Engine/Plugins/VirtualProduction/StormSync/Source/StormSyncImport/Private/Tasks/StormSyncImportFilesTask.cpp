// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/StormSyncImportFilesTask.h"

#include "StormSyncCoreDelegates.h"
#include "StormSyncImportLog.h"
#include "Subsystems/StormSyncImportSubsystem.h"

void FStormSyncImportFilesTask::Run()
{
	UE_LOGF(LogStormSyncImport, Display, "FStormSyncImportFilesTask::Run for %ls", *Filename);
	UStormSyncImportSubsystem::Get().PerformFileImport(Filename);
	FStormSyncCoreDelegates::OnFileImported.Broadcast(Filename);
}
