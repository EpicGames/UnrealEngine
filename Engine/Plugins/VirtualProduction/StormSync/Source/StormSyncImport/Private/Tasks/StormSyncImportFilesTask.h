// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Tasks/IStormSyncImportTask.h"

/** Import files from local exported buffer implementation for tasks that need delayed execution */
class FStormSyncImportFilesTask final : public IStormSyncImportSubsystemTask
{
public:
	explicit FStormSyncImportFilesTask(const FString& InFilename) :
		Filename(InFilename)
	{
	}

	virtual void Run() override;

private:
	FString Filename;
};