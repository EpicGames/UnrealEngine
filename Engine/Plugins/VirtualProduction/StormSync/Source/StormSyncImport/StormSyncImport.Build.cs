// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
public class StormSyncImport : ModuleRules
{
	public StormSyncImport(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"Slate",
				"SlateCore",
				"StormSyncCore",
				"StormSyncTransportCore",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorFramework",
					"MessageLog",
					"SourceControl",
					"StormSyncEditor",
					"UnrealEd",
				}
			);
		}
	}
}