// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetData.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "StormSyncCoreSettings.h"
#include "StormSyncCoreUtils.h"

#pragma warning(disable : 6011)

BEGIN_DEFINE_SPEC(FStormSyncCoreSpec, "StormSync.StormSyncCore", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	FString InvalidWidgetFixturePath = TEXT("/StormSync/Fixtures/Invalid");

	/** Map of fixture package name to expected list of dependencies. */
	const TMap<FName, TArray<FName>> ExpectedPackages = {
		{
			TEXT("/StormSync/Fixtures/Test_Layout/WB_Test_Layout"), {
				TEXT("/StormSync/Fixtures/Test_Layout/T_Layout_1"),
				TEXT("/StormSync/Fixtures/Test_Layout/T_Layout_2"),
				TEXT("/StormSync/Fixtures/Test_Layout/T_Layout_3"),
				TEXT("/StormSync/Fixtures/Test_Layout/T_Layout_4"),
				TEXT("/StormSync/Fixtures/Test_Layout/T_Layout_5"),
				TEXT("/StormSync/Fixtures/Test_Layout/T_Layout_6"),
				TEXT("/StormSync/Fixtures/Test_Layout/WB_Test_Layout"),
				TEXT("/StormSync/Fixtures/Test_Layout/WB_Test_Layout_Icon"),
				TEXT("/StormSync/Fixtures/Test_Layout/WB_Test_Layout_Icon_02"),
			}
		}
	};

	/** List of "top level" package names to check against in GetSyncFileModifiers test suite (maps keys of above map) */
	const TArray<FName> FilesList = {
		TEXT("/StormSync/Fixtures/Test_Layout/WB_Test_Layout")
	};

	/** Used with GetSyncFileModifiers test suite */
	TArray<FStormSyncFileDependency> CachedFileDependencies;

	/** Used with GetSyncFileModifiers test suite */
	FString BackupFilePath;

	/** Cache the value of bExportOnlyGameContent settings so that we can restore it */
	bool bPrevExportOnlyGameContent = false;

	/**
	 * Returns the list of dependencies we are expecting for a given fixture package name.
	 *
	 * @param PackageName Package name to lookup in ExpectedPackages map
	 */
	TArray<FName> GetExpectedDependenciesForAsset(const FName PackageName)
	{
		const TArray<FName>* ExpectedDependenciesPtr = ExpectedPackages.Find(PackageName);
		if (!ExpectedDependenciesPtr)
		{
			AddError(FString::Printf(TEXT("Unable to get expected list of dependencies (test suite internals)")));
			return {};
		}

		return *ExpectedDependenciesPtr;
	}

	/** Internal Equality Helper for arrays */
	bool TestArrayEqual(const TArray<FName>& Actual, const TArray<FName>& Expected)
	{
		const int32 ArrayANum = Actual.Num();
		if (!TestEqual(TEXT("Both arrays have same number of Elements"), ArrayANum, Expected.Num()))
		{
			return false;
		}

		for (int32 Index = 0; Index < ArrayANum; ++Index)
		{
			if (!TestTrue(TEXT("Expected array invalid index"), Expected.IsValidIndex(Index)))
			{
				return false;
			}

			if (!TestEqual(TEXT("Both arrays have same inner elements ()"), Actual[Index], Expected[Index]))
			{
				AddError(FString::Printf(TEXT("TestArrayEqual Actual: %s, Expected: %s"), *Actual[Index].ToString(), *Expected[Index].ToString()));
				return false;
			}
		}

		return true;
	}

	/** Internal Equality Helper for package file dependencies and an expected list of package names */
	bool TestEqualFileDependencies(const TArray<FStormSyncFileDependency>& ActualDependencies, const TArray<FName>& ExpectedDependencies)
	{
		if (!TestEqual(TEXT("Expected number of file dependencies"), ActualDependencies.Num(), ExpectedDependencies.Num()))
		{
			return false;
		}

		for (int32 Index = 0; Index < ActualDependencies.Num(); ++Index)
		{
			if (!TestTrue(TEXT("Expected array invalid index"), ExpectedDependencies.IsValidIndex(Index)))
			{
				return false;
			}

			FStormSyncFileDependency Dependency = ActualDependencies[Index];

			if (!TestEqual(TEXT("Both arrays have same inner elements ()"), Dependency.PackageName, ExpectedDependencies[Index]))
			{
				AddError(FString::Printf(TEXT("TestArrayEqual Actual: %s, Expected: %s"), *Dependency.PackageName.ToString(), *ExpectedDependencies[Index].ToString()));
				return false;
			}
		}

		return true;
	}

END_DEFINE_SPEC(FStormSyncCoreSpec)


void FStormSyncCoreSpec::Define()
{
	BeforeEach([this]()
	{
		// Since we rely on fixture files within /StormSync/Fixtures, make sure we allow export of non /Game content
		// or else, dependencies outside of /Game content will not be considered, thus failing the tests below

		UStormSyncCoreSettings* Settings = GetMutableDefault<UStormSyncCoreSettings>();
		check(Settings);

		bPrevExportOnlyGameContent = Settings->bExportOnlyGameContent;
		Settings->bExportOnlyGameContent= false;
	});

	Describe(TEXT("FStormSyncCoreUtils::GetAssetData"), [this]()
	{
		TArray<FName> FixturePackageNames;
		ExpectedPackages.GetKeys(FixturePackageNames);

		for (FName FixturePackageName : FixturePackageNames)
		{
			It(FString::Printf(TEXT("should return list of dependencies (%s)"), *FixturePackageName.ToString()), [this, FixturePackageName]()
			{
				TArray<FName> Dependencies;
				TArray<FAssetData> Assets;
				const bool bResult = FStormSyncCoreUtils::GetAssetData(FixturePackageName.ToString(), Assets, Dependencies);
				TestTrue(TEXT("GetAssetData result is true"), bResult);

				const TArray<FName> ExpectedDependencies = GetExpectedDependenciesForAsset(FixturePackageName);
				TestEqual(TEXT("Expected number of file dependencies"), Dependencies.Num(), ExpectedDependencies.Num());

				// GetAssetData does not sort list of dependencies, GetDependenciesForPackages does
				Dependencies.Sort(FNameLexicalLess());

				AddInfo(FString::Printf(TEXT("\t Dependencies: %d (Expected: %d)"), Dependencies.Num(), ExpectedDependencies.Num()));
				for (const FName& Dependency : Dependencies)
				{
					AddInfo(FString::Printf(TEXT("\t Dependency: %s"), *Dependency.ToString()));
				}
				
				TestArrayEqual(Dependencies, ExpectedDependencies);
			});
		}
	});
		
	Describe(TEXT("FStormSyncCoreUtils::GetDependenciesForPackages"), [this]()
	{
		It(TEXT("should fail if entries input is empty"), [this]()
		{
			FText ErrorText;
			const TArray<FName> Entries;
			TArray<FName> FilesToAdd;

			const bool bResult = FStormSyncCoreUtils::GetDependenciesForPackages(Entries, FilesToAdd, ErrorText);
			TestFalse(TEXT("GetDependenciesForPackages result is false"), bResult);
			TestTrue(TEXT("Expected number of file dependncies"), FilesToAdd.IsEmpty());
			// Might be better to not test against error text equality. Checking not empty might be enough.
			TestEqual(TEXT("GetDependenciesForPackages ErrorText is the expected one"), ErrorText.ToString(), TEXT("Provided PackageNames array is empty."));
		});

		It(TEXT("should fail if one entry is not a valid package name"), [this]()
		{
			FText ErrorText;
			TArray<FName> FilesToAdd;

			TArray<FName> Entries;
			Entries.Add(FName(*InvalidWidgetFixturePath));

			const bool bResult = FStormSyncCoreUtils::GetDependenciesForPackages(Entries, FilesToAdd, ErrorText);
			TestFalse(TEXT("GetDependenciesForPackages result is false"), bResult);
			TestTrue(TEXT("Expected number of file dependencies"), FilesToAdd.IsEmpty());
			// Might be better to not test against error text equality. Checking not empty might be enough.
			TestEqual(
				TEXT("GetDependenciesForPackages ErrorText is the expected one"),
				ErrorText.ToString(),
				TEXT("/StormSync/Fixtures/Invalid does not exist on disk.")
			);
		});

		TArray<FName> FixturePackageNames;
		ExpectedPackages.GetKeys(FixturePackageNames);

		for (FName FixturePackageName : FixturePackageNames)
		{
			It(FString::Printf(TEXT("should return list of dependencies (%s)"), *FixturePackageName.ToString()), [this, FixturePackageName]()
			{
				FText ErrorText;
				TArray<FName> FilesToAdd;

				TArray<FName> Entries;
				Entries.Add(FixturePackageName);

				const bool bResult = FStormSyncCoreUtils::GetDependenciesForPackages(Entries, FilesToAdd, ErrorText);
				TestTrue(TEXT("GetDependenciesForPackages result is true"), bResult);
				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());

				const TArray<FName> ExpectedDependencies = GetExpectedDependenciesForAsset(FixturePackageName);
				TestEqual(TEXT("Expected number of file dependencies"), FilesToAdd.Num(), ExpectedDependencies.Num());
				TestArrayEqual(FilesToAdd, ExpectedDependencies);
			});
		}
	});

	Describe(TEXT("FStormSyncCoreUtils::GetAvaFileDependenciesForPackages"), [this]
	{
		It(TEXT("should fail if entries input is empty"), [this]()
		{
			FText ErrorText;
			const TArray<FName> Entries;

			TArray<FStormSyncFileDependency> FileDependencies;
			const bool bResult = FStormSyncCoreUtils::GetAvaFileDependenciesForPackages(Entries, FileDependencies, ErrorText);
			TestFalse(TEXT("GetAvaFileDependenciesForPackages result is false"), bResult);
			TestTrue(TEXT("Expected number of file dependencies"), FileDependencies.IsEmpty());
			// Might be better to not test against error text equality. Checking not empty might be enough.
			TestEqual(TEXT("GetAvaFileDependenciesForPackages ErrorText is the expected one"), ErrorText.ToString(), TEXT("Provided PackageNames array is empty."));
		});

		It(TEXT("should fail if one entry is not a valid package name"), [this]()
		{
			FText ErrorText;

			TArray<FName> Entries;
			Entries.Add(FName(*InvalidWidgetFixturePath));

			TArray<FStormSyncFileDependency> FileDependencies;
			const bool bResult = FStormSyncCoreUtils::GetAvaFileDependenciesForPackages(Entries, FileDependencies, ErrorText);
			TestFalse(TEXT("GetAvaFileDependenciesForPackages result is false"), bResult);
			TestTrue(TEXT("Expected number of file dependencies"), FileDependencies.IsEmpty());
			// Might be better to not test against error text equality. Checking not empty might be enough.
			TestEqual(
				TEXT("GetAvaFileDependenciesForPackages ErrorText is the expected one"),
				ErrorText.ToString(),
				TEXT("/StormSync/Fixtures/Invalid does not exist on disk.")
			);
		});

		TArray<FName> FixturePackageNames;
		ExpectedPackages.GetKeys(FixturePackageNames);

		for (FName FixturePackageName : FixturePackageNames)
		{
			It(FString::Printf(TEXT("should return list of dependencies (%s)"), *FixturePackageName.ToString()), [this, FixturePackageName]()
			{
				FText ErrorText;

				TArray<FName> Entries;
				Entries.Add(FixturePackageName);

				TArray<FStormSyncFileDependency> FileDependencies;
				const bool bResult = FStormSyncCoreUtils::GetAvaFileDependenciesForPackages(Entries, FileDependencies, ErrorText);
				TestTrue(TEXT("GetDependenciesForPackages result is true"), bResult);
				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());

				const TArray<FName> ExpectedDependencies = GetExpectedDependenciesForAsset(FixturePackageName);
				TestEqual(TEXT("Expected number of file dependencies"), FileDependencies.Num(), ExpectedDependencies.Num());
				// Check against expected package names
				TestEqualFileDependencies(FileDependencies, ExpectedDependencies);

				for (FStormSyncFileDependency& ActualDependency : FileDependencies)
				{
					AddInfo(FString::Printf(TEXT("FileDependency - %s"), *ActualDependency.ToString()));

					FName PackageName = ActualDependency.PackageName;

					FStormSyncFileDependency ExpectedDependency;
					ExpectedDependency.PackageName = PackageName;

					FString PackageFilepath;
					if (!FPackageName::DoesPackageExist(PackageName.ToString(), &PackageFilepath))
					{
						AddError(FString::Printf(TEXT("Attempting to read file with \"%s\" which is not a file."), *PackageName.ToString()));
						return;
					}

					const TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileReader(*PackageFilepath));
					if (!FileHandle.IsValid())
					{
						AddError(FString::Printf(TEXT("Invalid file handle \"%s\""), *PackageFilepath));
						return;
					}

					ExpectedDependency.FileSize = FileHandle->TotalSize();
					ExpectedDependency.Timestamp = IFileManager::Get().GetTimeStamp(*PackageFilepath).ToUnixTimestamp();

					// Note: Consider another hashing algorithm (as per Matt's suggestion)
					FMD5Hash FileHash = FMD5Hash::HashFile(*PackageFilepath);
					ExpectedDependency.FileHash = LexToString(FileHash);

					AddInfo(FString::Printf(TEXT("ExpectedDependency - %s"), *ExpectedDependency.ToString()));

					TestEqual(TEXT("Package Name is correct"), ActualDependency.PackageName, ExpectedDependency.PackageName);
					TestEqual(TEXT("FileSize is correct"), ActualDependency.FileSize, ExpectedDependency.FileSize);
					TestEqual(TEXT("Timestamp is correct"), ActualDependency.Timestamp, ExpectedDependency.Timestamp);
					TestEqual(TEXT("FileHash is correct"), ActualDependency.FileHash, ExpectedDependency.FileHash);

					// Close the file
					FileHandle->Close();
				}
			});
		}
	});

	Describe(TEXT("FStormSyncCoreUtils::CreatePakBufferWithDependencies"), [this]
	{
		It(TEXT("should fail if entries input is empty"), [this]()
		{
			FText ErrorText;
			FStormSyncBuffer PakBuffer;
			FMemoryWriter64 PakArchiveWriter(PakBuffer);
			const TArray<FName> PackageNames;

			const bool bResult = FStormSyncCoreUtils::CreatePakBufferWithDependencies(PackageNames, PakArchiveWriter, ErrorText);

			TestFalse(TEXT("CreatePakBufferWithDependencies result is false"), bResult);
			TestEqual(TEXT("CreatePakBufferWithDependencies ErrorText is the expected one"), ErrorText.ToString(), TEXT("Provided PackageNames array is empty."));
			TestEqual(TEXT("CreatePakBufferWithDependencies PakBuffer remains empty"), PakBuffer.Num(), static_cast<int64>(0));
		});

		TArray<FName> FixturePackageNames;
		ExpectedPackages.GetKeys(FixturePackageNames);

		for (FName FixturePackageName : FixturePackageNames)
		{
			It(FString::Printf(TEXT("should create package descriptor (%s)"), *FixturePackageName.ToString()), [this, FixturePackageName]()
			{
				FText ErrorText;

				FStormSyncBuffer PakBuffer;
				FMemoryWriter64 PakArchiveWriter(PakBuffer);

				TArray<FName> PackageNames;
				PackageNames.Add(FixturePackageName);

				FStormSyncPackageDescriptor PackageDescriptor;
				PackageDescriptor.Name = TEXT("TestPak");
				PackageDescriptor.Description = TEXT("Example description");

				const FStormSyncCoreUtils::FOnFileAdded Delegate = FStormSyncCoreUtils::FOnFileAdded::CreateLambda([&PackageDescriptor](const FStormSyncFileDependency& FileDependency)
				{
					PackageDescriptor.Dependencies.Add(FileDependency);
				});

				const bool bResult = FStormSyncCoreUtils::CreatePakBufferWithDependencies(PackageNames, PakArchiveWriter, ErrorText, Delegate);

				TestTrue(TEXT("CreatePakBufferWithDependencies result is true"), bResult);
				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());


				TestEqual(TEXT("Package descriptor name is correct"), PackageDescriptor.Name, TEXT("TestPak"));
				TestEqual(TEXT("Package descriptor version is correct"), PackageDescriptor.Version, TEXT("1.0.0"));
				TestEqual(TEXT("Package descriptor description is correct"), PackageDescriptor.Description, TEXT("Example description"));
				TestEqual(TEXT("Package descriptor author is correct"), PackageDescriptor.Author, FPlatformProcess::UserName());
				TestTrue(TEXT("Pak buffer is non empty"), !PakBuffer.IsEmpty());

				const TArray<FStormSyncFileDependency> Dependencies = PackageDescriptor.Dependencies;
				const TArray<FName> ExpectedDependencies = GetExpectedDependenciesForAsset(FixturePackageName);

				TestEqual(TEXT("Expected number of file dependencies"), Dependencies.Num(), ExpectedDependencies.Num());
				TestEqualFileDependencies(Dependencies, ExpectedDependencies);
			});
		}
	});

	Describe(TEXT("FStormSyncCoreUtils::CreatePakBuffer"), [this]
	{
		It(TEXT("should fail if entries input is empty"), [this]()
		{
			FText ErrorText;
			const TArray<FName> PackageNames;

			FStormSyncBuffer PakBuffer;
			FMemoryWriter64 PakArchiveWriter(PakBuffer);
			const bool bResult = FStormSyncCoreUtils::CreatePakBuffer(PackageNames, PakArchiveWriter, ErrorText);

			TestFalse(TEXT("CreatePakBuffer result is false"), bResult);
			TestEqual(TEXT("CreatePakBuffer ErrorText is the expected one"), ErrorText.ToString(), TEXT("Provided PackageNames array is empty."));
		});

		TArray<FName> FixturePackageNames;
		ExpectedPackages.GetKeys(FixturePackageNames);

		for (FName FixturePackageName : FixturePackageNames)
		{
			It(FString::Printf(TEXT("should create pak buffer (%s)"), *FixturePackageName.ToString()), [this, FixturePackageName]()
			{
				const FString AvaPackageName = TEXT("TestPak");

				FText ErrorText;

				TArray<FName> PackageNames;
				PackageNames.Add(FixturePackageName);

				TArray<FName> PackageDependencies;
				if (!FStormSyncCoreUtils::GetDependenciesForPackages(PackageNames, PackageDependencies, ErrorText))
				{
					AddError(FString::Printf(TEXT("GetDependenciesForPackages Error: %s"), *ErrorText.ToString()));
					return;
				}

				FStormSyncBuffer PakBuffer;
				FMemoryWriter64 PakArchiveWriter(PakBuffer);

				TArray<FStormSyncFileDependency> SuccessfullyPackedFiles;
				const FStormSyncCoreUtils::FOnFileAdded Delegate = FStormSyncCoreUtils::FOnFileAdded::CreateLambda([&SuccessfullyPackedFiles](const FStormSyncFileDependency& FileDependency)
				{
					SuccessfullyPackedFiles.Add(FileDependency);
				});

				const bool bResult = FStormSyncCoreUtils::CreatePakBuffer(PackageDependencies, PakArchiveWriter, ErrorText, Delegate);

				TestTrue(TEXT("CreatePakBufferWithDependencies result is true"), bResult);
				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());

				TestTrue(TEXT("Pak buffer is non empty"), !PakBuffer.IsEmpty());

				const TArray<FName> ExpectedDependencies = GetExpectedDependenciesForAsset(FixturePackageName);

				TestEqual(TEXT("Expected number of file dependencies"), SuccessfullyPackedFiles.Num(), ExpectedDependencies.Num());
				TestEqualFileDependencies(SuccessfullyPackedFiles, ExpectedDependencies);
			});
		}
	});

	Describe(TEXT("FStormSyncCoreUtils::ExtractPakBuffer"), [this]
	{
		TArray<FName> FixturePackageNames;
		ExpectedPackages.GetKeys(FixturePackageNames);

		for (FName FixturePackageName : FixturePackageNames)
		{
			It(FString::Printf(TEXT("should extract package descriptor (%s)"), *FixturePackageName.ToString()), [this, FixturePackageName]()
			{
				FText ErrorText;
				FStormSyncBuffer PakBuffer;
				FMemoryWriter64 PakArchiveWriter(PakBuffer);

				TArray<FName> PackageNames;
				PackageNames.Add(FixturePackageName);

				FStormSyncPackageDescriptor PackageDescriptor;
				PackageDescriptor.Name = TEXT("TestPak");

				const FStormSyncCoreUtils::FOnFileAdded Delegate = FStormSyncCoreUtils::FOnFileAdded::CreateLambda([&PackageDescriptor](const FStormSyncFileDependency& FileDependency)
				{
					PackageDescriptor.Dependencies.Add(FileDependency);
				});

				if (!FStormSyncCoreUtils::CreatePakBufferWithDependencies(PackageNames, PakArchiveWriter, ErrorText, Delegate))
				{
					AddError(FString::Printf(TEXT("CreatePakBufferWithDependencies Error: %s"), *ErrorText.ToString()));
					return;
				}

				TArray<FText> Errors;
				TMap<FString, FString> SuccessfullyExtractedPackages;
				FStormSyncCoreExtractArgs ExtractArgs;
				ExtractArgs.OnPakPreExtract.BindLambda([this](const int32 FileCount)
				{
					AddInfo(FString::Printf(TEXT("\tOnPakPreExtract %d"), FileCount));
				});

				ExtractArgs.OnPakPostExtract.BindLambda([this](const int32 FileCount)
				{
					AddInfo(FString::Printf(TEXT("\tOnPakPostExtract %d"), FileCount));
				});

				ExtractArgs.OnFileExtract.BindLambda([this, &SuccessfullyExtractedPackages](const FStormSyncFileDependency& FileDependency, const FString DestFilepath, const FStormSyncBufferPtr& FileBuffer)
				{
					AddInfo(FString::Printf(TEXT("\tOnFileExtract")));
					AddInfo(FString::Printf(TEXT("\t\tPackageName: %s"), *FileDependency.PackageName.ToString()));
					AddInfo(FString::Printf(TEXT("\t\tDestFilepath: %s"), *DestFilepath));
					AddInfo(FString::Printf(TEXT("\t\tFileSize: %lld"), FileDependency.FileSize));
					AddInfo(FString::Printf(TEXT("\t\tFileBuffer valid: %s"), FileBuffer.IsValid() ? TEXT("valid") : TEXT("invalid")));
					if (FileBuffer.IsValid())
					{
						AddInfo(FString::Printf(TEXT("\t\tFileBuffer size: %lld"), FileBuffer->Num()));
					}

					SuccessfullyExtractedPackages.Add(FileDependency.PackageName.ToString(), DestFilepath);
				});

				FMemoryReaderView PakArchiveReader(PakBuffer);
				const bool bResult = FStormSyncCoreUtils::ExtractPakBuffer(PakArchiveReader, ExtractArgs, Errors);

				TestTrue(TEXT("ExtractPakBuffer result is true"), bResult);
				TestEqual(TEXT("ExtractPakBuffer SuccessfullyExtractedPackages number is equal to the original file numbers"), SuccessfullyExtractedPackages.Num(), PackageDescriptor.Dependencies.Num());
				TestTrue(TEXT("ExtractPakBuffer errors is empty"), Errors.IsEmpty());

				for (auto Entry : SuccessfullyExtractedPackages)
				{
					FName PackageName = FName(*Entry.Key);
					FString DestFilepath = Entry.Value;

					FStormSyncFileDependency* FileDependency = PackageDescriptor.Dependencies.FindByPredicate([PackageName](const FStormSyncFileDependency& Item)
					{
						return Item.PackageName == PackageName;
					});

					if (!TestTrue(TEXT("Found expected file dependency in SuccessfullyExtractedPackages"), FileDependency != nullptr))
					{
						AddError(FString::Printf(TEXT("Wasn't able to get %s from original list of file dependencies"), *PackageName.ToString()));
					}
					else
					{
						FText FailReason;
						FString ExpectedDestFilepath = FileDependency->GetDestFilepath(FileDependency->PackageName, FailReason) + TEXT(".uasset");
						TestEqual(TEXT("Destination filepath is correct"), DestFilepath, ExpectedDestFilepath);		
					}
				}
			});
		}
	});

	Describe(TEXT("FStormSyncCoreUtils::GetHumanReadableByteSize"), [this]
	{
		It(TEXT("should return human readable size"), [this]
		{
			TestEqual(TEXT("Size 1500"), FStormSyncCoreUtils::GetHumanReadableByteSize(1500), TEXT("1.46 KB"));
			TestEqual(TEXT("Size 1.00 GB"), FStormSyncCoreUtils::GetHumanReadableByteSize(1024 * 1024 * 1024), TEXT("1.00 GB"));

			TestEqual(TEXT("Size 1892830"), FStormSyncCoreUtils::GetHumanReadableByteSize(1892830), TEXT("1.80 MB"));
			TestEqual(TEXT("Size 424592198"), FStormSyncCoreUtils::GetHumanReadableByteSize(424592198), TEXT("404.92 MB"));
		});
	});

	Describe(TEXT("FStormSyncCoreUtils::GetSyncFileModifiers"), [this]
	{
		BeforeEach([this]
		{
			FText ErrorText;

			// This is our base list of files with initial state
			TArray<FStormSyncFileDependency> LocalFileDependencies;
			if (!FStormSyncCoreUtils::GetAvaFileDependenciesForPackages(FilesList, LocalFileDependencies, ErrorText))
			{
				AddError(FString::Printf(TEXT("Wasn't able to compute file depdencies for file list")));
				return;
			}

			CachedFileDependencies = MoveTemp(LocalFileDependencies);
			for (const FStormSyncFileDependency& FileDependency : CachedFileDependencies)
			{
				AddInfo(FString::Printf(TEXT("CachedFileDependency: %s"), *FileDependency.ToString()));
			}
		});

		It(TEXT("should return no modifications if both list of files are the same state"), [this]
		{
			if (CachedFileDependencies.IsEmpty())
			{
				AddError(FString::Printf(TEXT("Wasn't able to compute file depdencies")));
				return;
			}

			// Since we're diffing against local files, we expect no diff between the two
			const TArray<FStormSyncFileModifierInfo> ModifierInfos = FStormSyncCoreUtils::GetSyncFileModifiers(FilesList, CachedFileDependencies);
			TestTrue(TEXT("ModifierInfos is empty"), ModifierInfos.IsEmpty());
		});

		It(TEXT("should return some modifications if both list of files have different states"), [this]
		{
			if (CachedFileDependencies.IsEmpty())
			{
				AddError(FString::Printf(TEXT("Wasn't able to compute file dependencies")));
				return;
			}

			// TODO: Ensure asset registry is up to date.
			//
			// This is not necessary running tests in editor via Session Frontend, but it appears asset registry is not in the same state
			// when tests are run from command line. Currently test is failing because GetAvaFileDependenciesForPackages fails to load inner
			// dependencies when run from command line
			//
			// For now, skip the test in this case.
			if (CachedFileDependencies.Num() <= 1)
			{
				AddWarning(TEXT("Wasn't able to compute file depdencies (AssetRegistry not up to date when run from CLI)"));
				return;
			}

			TArray<FStormSyncFileDependency> DiffingFileDependencies;
			DiffingFileDependencies.Append(CachedFileDependencies);
			
			FStormSyncFileDependency& TopLvlFile = DiffingFileDependencies[0];
			FStormSyncFileDependency& FirstFile = DiffingFileDependencies[1];

			// Slightly change file size to mark it dirty
			TopLvlFile.FileSize += 1;

			// Slightly change file hash to mark it dirty
			FirstFile.FileHash += TEXT("__dirty");

			// We expect two modifications with overwrite modifier operation
			const TArray<FStormSyncFileModifierInfo> ModifierInfos = FStormSyncCoreUtils::GetSyncFileModifiers(FilesList, DiffingFileDependencies);
			TestEqual(TEXT("ModifierInfos ..."), ModifierInfos.Num(), 2);

			if (!ModifierInfos.IsValidIndex(0) || !ModifierInfos.IsValidIndex(1))
			{
				AddError(FString::Printf(TEXT("ModifierInfos invalid indexes, most likely empty")));
				return;
			}

			const FStormSyncFileModifierInfo FirstModifier = ModifierInfos[0];
			const FStormSyncFileModifierInfo SecondModifier = ModifierInfos[1];

			TestEqual(TEXT("ModifierInfos FirstModifier Op"), FirstModifier.ModifierOperation, EStormSyncModifierOperation::Overwrite);
			TestEqual(TEXT("ModifierInfos SecondModifier Op"), SecondModifier.ModifierOperation, EStormSyncModifierOperation::Overwrite);
		});

		It(TEXT("should return addition modification is a file is missing"), [this]
		{
			if (CachedFileDependencies.IsEmpty())
			{
				AddError(FString::Printf(TEXT("Wasn't able to compute file depdencies")));
				return;
			}

			TArray<FStormSyncFileDependency> DiffingFileDependencies;
			DiffingFileDependencies.Append(CachedFileDependencies);

			const FString DummyFile = TEXT("/StormSync/Fixtures/Test_Layout/WB_Dummy");

			FStormSyncFileDependency DummyDependency;
			DummyDependency.PackageName = FName(*DummyFile);
			DiffingFileDependencies.Add(DummyDependency);

			// We expect one modification with addition modifier operation
			const TArray<FStormSyncFileModifierInfo> ModifierInfos = FStormSyncCoreUtils::GetSyncFileModifiers(FilesList, DiffingFileDependencies);
			TestEqual(TEXT("ModifierInfos ..."), ModifierInfos.Num(), 1);

			if (!ModifierInfos.IsValidIndex(0))
			{
				AddError(FString::Printf(TEXT("ModifierInfos invalid indexes, most likely empty")));
				return;
			}

			const FStormSyncFileModifierInfo FirstModifier = ModifierInfos[0];
			TestEqual(TEXT("ModifierInfos FirstModifier Op"), FirstModifier.ModifierOperation, EStormSyncModifierOperation::Addition);
			TestEqual(TEXT("ModifierInfos FirstModifier File"), FirstModifier.FileDependency.PackageName.ToString(), DummyFile);
		});

		Describe(TEXT("should return addition modification if top lvl file is missing"), [this]
		{
			BeforeEach([this]
			{
				const FStormSyncFileDependency TopLvlFile = CachedFileDependencies[0];

				FString PackageFilepath;
				if (!FPackageName::DoesPackageExist(TopLvlFile.PackageName.ToString(), &PackageFilepath))
				{
					AddError(FString::Printf(TEXT("Wasn't able to compute package path for %s"), *TopLvlFile.PackageName.ToString()));
					return;
				}

				// Create a backup to be able to restore it on tear down
				BackupFilePath = PackageFilepath + TEXT(".backup");
				if (IFileManager::Get().Copy(*BackupFilePath, *PackageFilepath) != COPY_OK)
				{
					AddError(FString::Printf(TEXT("Wasn't able to copy file %s to %s"), *PackageFilepath, *BackupFilePath));
					return;
				}

				// Temporary remove top lvl file
				if (!IFileManager::Get().Delete(*PackageFilepath, false, true, false))
				{
					AddError(FString::Printf(TEXT("Wasn't able to delete file %s"), *PackageFilepath));
				}
			});

			It(TEXT("should return addition modification if top lvl file is missing"), [this]
			{
				// This is the use case that wasn't covered properly by previous implementation
				if (CachedFileDependencies.IsEmpty())
				{
					AddError(FString::Printf(TEXT("Wasn't able to compute file depdencies")));
					return;
				}

				const FStormSyncFileDependency TopLvlFile = CachedFileDependencies[0];

				// We'll get a warning about missing file we just deleted in BeforeEach
				AddExpectedError(FString::Printf(TEXT("GetAssetData failed to load assets for %s"), *TopLvlFile.PackageName.ToString()));

				// We expect one modification with addition modifier operation
				const TArray<FStormSyncFileModifierInfo> ModifierInfos = FStormSyncCoreUtils::GetSyncFileModifiers(FilesList, CachedFileDependencies);
				TestEqual(TEXT("ModifierInfos ..."), ModifierInfos.Num(), 1);

				if (!ModifierInfos.IsValidIndex(0))
				{
					AddError(FString::Printf(TEXT("ModifierInfos invalid indexes, most likely empty")));
					return;
				}

				const FStormSyncFileModifierInfo FirstModifier = ModifierInfos[0];
				AddInfo(FString::Printf(TEXT("Got modifier: %s"), *FirstModifier.ToString()));
				TestEqual(TEXT("ModifierInfos FirstModifier Op"), FirstModifier.ModifierOperation, EStormSyncModifierOperation::Addition);
				TestEqual(TEXT("ModifierInfos FirstModifier File"), FirstModifier.FileDependency.PackageName, TopLvlFile.PackageName);
			});

			AfterEach([this]
			{
				FString PackageFilePath = BackupFilePath;
				PackageFilePath.RemoveFromEnd(TEXT(".backup"));

				if (IFileManager::Get().Copy(*PackageFilePath, *BackupFilePath, true, true) != COPY_OK)
				{
					AddError(FString::Printf(TEXT("Wasn't able to copy file %s to %s"), *BackupFilePath, *PackageFilePath));
					return;
				}

				// Remove backup file
				if (!IFileManager::Get().Delete(*BackupFilePath, false, true, false))
				{
					AddError(FString::Printf(TEXT("Wasn't able to delete file %s"), *BackupFilePath));
				}
			});
		});

		AfterEach([this]
		{
			CachedFileDependencies.Empty();
		});
	});

	AfterEach([this]()
	{
		UStormSyncCoreSettings* Settings = GetMutableDefault<UStormSyncCoreSettings>();
		check(Settings);

		Settings->bExportOnlyGameContent = bPrevExportOnlyGameContent;
	});
}

#pragma warning(default : 6011)
