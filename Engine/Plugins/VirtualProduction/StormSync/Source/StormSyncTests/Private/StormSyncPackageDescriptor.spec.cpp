// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/FileManager.h"
#include "IStormSyncDrivesModule.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncDrivesSettings.h"

BEGIN_DEFINE_SPEC(FStormSyncPackageDescriptorSpec, "StormSync.StormSyncCore", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	FString TestMountPoint = TEXT("/StormSyncTests_Temp");
	FString TestMountDirectory = FPaths::ProjectSavedDir() / TEXT("StormSyncTests");

	/**
	 * Test helper to return the fully qualified path for a Plugin Content directory if it is installed in current project,
	 * or an empty string if not.
	 *
	 * Note: Was part of FStormSyncFileDependency API but not used anymore since GetDestFilepath is now using
	 * FPackageName::TryConvertLongPackageNameToFilename()
	 */
	static FString GetPluginContentPath(const FString& InPluginName)
	{
		FString ContentPath;

		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(InPluginName);
		if (Plugin.IsValid() && Plugin->IsEnabled())
		{
			return Plugin->GetContentDir();
		}

		return ContentPath;
	}

END_DEFINE_SPEC(FStormSyncPackageDescriptorSpec)

void FStormSyncPackageDescriptorSpec::Define()
{
	Describe(TEXT("FStormSyncFileDependency::GetDestFilepath"), [this]()
	{
		BeforeEach([this]()
		{
			if (!IFileManager::Get().DirectoryExists(*TestMountDirectory))
			{
				if (!IFileManager::Get().MakeDirectory(*TestMountDirectory))
				{
					AddError(FString::Printf(TEXT("Failed to create directory %s for automated tests"), *TestMountDirectory));
					return;
				}
			}

			// Temporarily mount /StormSyncTests_Temp.
			FText ErrorText;
			if (!IStormSyncDrivesModule::Get().RegisterMountPoint(FStormSyncMountPointConfig(TestMountPoint, TestMountDirectory), ErrorText))
			{
				AddError(FString::Printf(TEXT("Failed to mount %s - Error: %s"), *TestMountPoint, *ErrorText.ToString()));
			}
		});

		It(TEXT("should return expected path, if it is a package from a valid mount point"), [this]()
		{
			// Map of PackageNames => ExpectedFilePath
			TMap<FName, FString> TestCases = {
				{TEXT("/Game/Foo/Bar"), FPaths::ProjectContentDir() / TEXT("Foo/Bar")},
				{TEXT("/StormSync/Foo/Bar"), GetPluginContentPath(TEXT("StormSync")) / TEXT("Foo/Bar")},
				{TEXT("/StormSyncTests_Temp/Foo/Bar"), TestMountDirectory / TEXT("Foo/Bar")}
			};

			for (const TPair<FName, FString>& TestCase : TestCases)
			{
				FName PackageName = TestCase.Key;
				FString ExpectedFilepath = TestCase.Value;

				AddInfo(FString::Printf(TEXT("Testing PackageName: %s, ExpectedFilepath: %s"), *PackageName.ToString(), *ExpectedFilepath));

				FText ErrorText;
				FString DestFilepath = FStormSyncFileDependency::GetDestFilepath(PackageName, ErrorText);

				TestTrue(TEXT("ErrorText is empty"), ErrorText.IsEmpty());
				if (!ErrorText.IsEmpty())
				{
					AddError(ErrorText.ToString());
				}

				TestTrue(FString::Printf(TEXT("Has expected filepath for %s"), *PackageName.ToString()), FPaths::IsSamePath(DestFilepath, ExpectedFilepath));
			}
		});

		It(TEXT("should reject package paths that escape mounted points via path traversal (UE-368657)"), [this]()
		{
			// A malformed/malicious .spak file can contain PackageName entries containing ../ sequences to escape
			// the project's mount points and write to arbitrary locations on disk. GetDestFilepath
			// must return an empty path and a populated failure reason for any such input.
			const TArray<FName> TestPackageNames = {
				TEXT("/Game/../../Foo/Bar"),
				TEXT("/StormSyncTests_Temp/../../Foo/Bar"),
				TEXT("/StormSync/../../Foo/Bar")
			};
			for (const FName& PackageName : TestPackageNames)
			{
				AddInfo(FString::Printf(TEXT("Testing malicious/malformed PackageName: %s"), *PackageName.ToString()));

				FText ErrorText;
				const FString DestFilepath = FStormSyncFileDependency::GetDestFilepath(PackageName, ErrorText);
				TestTrue(FString::Printf(TEXT("DestFilepath is empty for traversal attempt %s"), *PackageName.ToString()), DestFilepath.IsEmpty());
				TestFalse(FString::Printf(TEXT("ErrorText is set for traversal attempt %s"), *PackageName.ToString()), ErrorText.IsEmpty());
			}
		});

		AfterEach([this]()
		{
			FText ErrorText;
			if (!IStormSyncDrivesModule::Get().UnregisterMountPoint(FStormSyncMountPointConfig(TestMountPoint, TestMountDirectory), ErrorText))
			{
				AddError(FString::Printf(TEXT("Failed to unmount %s - Error: %s"), *TestMountDirectory, *ErrorText.ToString()));
				return;
			}

			if (IFileManager::Get().DirectoryExists(*TestMountDirectory))
			{
				if (!IFileManager::Get().DeleteDirectory(*TestMountDirectory))
				{
					AddError(FString::Printf(TEXT("Failed to delete directory %s for automated tests"), *TestMountDirectory));
				}
			}
		});
	});
}
