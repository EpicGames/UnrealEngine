// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncPullMessageService.h"

#include "IStormSyncTransportClientModule.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/PackageName.h"
#include "Socket/StormSyncTransportClientSocket.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncTransportClientEndpoint.h"
#include "StormSyncTransportClientLog.h"
#include "StormSyncTransportNetworkUtils.h"

#define LOCTEXT_NAMESPACE "StormSyncPullMessageService"

FStormSyncPullMessageService::FStormSyncPullMessageService(const TSharedPtr<IStormSyncTransportLocalEndpoint, ESPMode::ThreadSafe> InLocalEndpoint)
	: LocalEndpointWeak(InLocalEndpoint)
{
}

FStormSyncPullMessageService::~FStormSyncPullMessageService()
{
	PendingResponseHandlers.Empty();
}

void FStormSyncPullMessageService::InitializeMessageEndpoint(FMessageEndpointBuilder& InEndpointBuilder)
{
	InEndpointBuilder
		.Handling<FStormSyncTransportPullRequest>(this, &FStormSyncPullMessageService::HandlePullRequestMessage)
		.Handling<FStormSyncTransportPullResponse>(this, &FStormSyncPullMessageService::HandlePullResponseMessage);
}

void FStormSyncPullMessageService::RequestPullPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPullComplete& InDoneDelegate)
{
	FMessageAddress MessageAddress = InRemoteAddress;
	TArray<FName> LocalPackageNames = InPackageNames;
	FStormSyncPackageDescriptor LocalPackageDescriptor = InPackageDescriptor;
	
	TWeakPtr<FStormSyncPullMessageService, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));

	constexpr bool bShouldValidatePackages = false;
	FStormSyncCoreUtils::GetAvaFileDependenciesAsync(InPackageNames, bShouldValidatePackages)
	.Then([LocalWeakThis, PackageNames = MoveTemp(LocalPackageNames), PackageDescriptor = MoveTemp(LocalPackageDescriptor), Recipient = MoveTemp(MessageAddress), InDoneDelegate](const TFuture<TArray<FStormSyncFileDependency>> Result)
	{
		const TSharedPtr<FStormSyncPullMessageService, ESPMode::ThreadSafe> LocalThis = LocalWeakThis.Pin();
		if (!LocalThis)
		{
			return;
		}
		
		const TArray<FStormSyncFileDependency> FileDependencies = Result.Get();

		// Validate we have at least some file dependencies that were resolved
		if (FileDependencies.IsEmpty())
		{
			const FText ErrorText = LOCTEXT("FileDependencies_Error", "Error before pulling: File dependencies array is empty, something went wrong");
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::RequestPullPackages - %ls", *ErrorText.ToString());
			
			FStormSyncTransportPullResponse Payload = LocalThis->CreatePullResponsePayload(PackageNames, PackageDescriptor);
			Payload.Status = EStormSyncResponseResult::Error;
			Payload.StatusText = ErrorText;
			InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncTransportPullResponse>(Payload));
			return;
		}

		const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = LocalThis->GetMessageEndpoint();
		if (!MessageEndpoint.IsValid())
		{
			const FText ErrorText = LOCTEXT("InvalidMessageEndpoint_Error", "Error before pulling: Unable to get message endpoint");
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::RequestPullPackages - %ls", *ErrorText.ToString());

			FStormSyncTransportPullResponse Payload = LocalThis->CreatePullResponsePayload(PackageNames, PackageDescriptor);
			Payload.Status = EStormSyncResponseResult::Error;
			Payload.StatusText = ErrorText;
			InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncTransportPullResponse>(Payload));
			return;
		}

		// Build sync request message that is going to be sent over the network for a specific recipient
		TUniquePtr<FStormSyncTransportPullRequest> PullRequestMessage(FMessageEndpoint::MakeMessage<FStormSyncTransportPullRequest>(PackageNames, PackageDescriptor));
		if (!PullRequestMessage.IsValid())
		{
			const FText ErrorText = LOCTEXT("InvalidPullRequestMessage_Error", "Error before pulling: Pull request message is invalid");
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::RequestPullPackages - %ls", *ErrorText.ToString());

			FStormSyncTransportPullResponse Payload = LocalThis->CreatePullResponsePayload(PackageNames, PackageDescriptor);
			Payload.Status = EStormSyncResponseResult::Error;
			Payload.StatusText = ErrorText;
			InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncTransportPullResponse>(Payload));
			
			return;
		}
		
		// Store completion callback for later invocation when receiving back a pull response
		LocalThis->AddResponseHandler(PullRequestMessage->MessageId, InDoneDelegate);

		PullRequestMessage->PackageDescriptor.Dependencies = FileDependencies;

		// Fill in Pull Request Message with network info for this device
		PullRequestMessage->HostName = FStormSyncTransportNetworkUtils::GetServerName();
		PullRequestMessage->HostAddress = FStormSyncTransportNetworkUtils::GetCurrentTcpServerEndpointAddress();
		PullRequestMessage->HostAdapterAddresses = FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses();

		UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::RequestPullPackages - FileDependencies: %d", FileDependencies.Num());
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::RequestPullPackages - Message: %ls", *PullRequestMessage->ToString());
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::RequestPullPackages - Syncing package descriptor %ls", *PullRequestMessage->PackageDescriptor.ToString());

		MessageEndpoint->Send(PullRequestMessage.Release(), Recipient);
	});
}

void FStormSyncPullMessageService::AbortPullRequest(const FGuid& InPullRequestId)
{
	FScopeLock ScopeLock(&HandlersCriticalSection);
	PendingResponseHandlers.Remove(InPullRequestId);
}

void FStormSyncPullMessageService::AddResponseHandler(const FGuid& InId, const FOnStormSyncPullComplete& InDelegate)
{
	FScopeLock ScopeLock(&HandlersCriticalSection);
	PendingResponseHandlers.Add(InId, InDelegate);
}

void FStormSyncPullMessageService::HandlePullRequestMessage(const FStormSyncTransportPullRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::HandlePullRequestMessage - Received pull request Message from %ls (%ls)", *InMessageContext->GetSender().ToString(), *InMessage.ToString());

	// We received a pull request message from a remote, which contains the full list of "file dependencies" with file state (hash, timestamp), the remote has and wants an update
	// We have to compare this list with the state of these files locally, and send back a buffer with just the files in a mismatched state

	FStormSyncTransportPullRequest PullRequestMessage = InMessage;
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> LocalMessageEndpoint = GetMessageEndpoint();
	FMessageAddress SenderAddress = InMessageContext->GetSender();
	
	TWeakPtr<FStormSyncPullMessageService, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));

	FText ValidationErrorText;
	// Validate list of package names remote wants to pull upfront. Remote might request a pull on top level packages that are not present on disk on this editor instance
	if (!ValidateIncomingPullMessage(InMessage, ValidationErrorText))
	{
		FStormSyncTransportPullResponse Payload = CreatePullResponsePayload(InMessage);
		Payload.Status = EStormSyncResponseResult::Error;
		Payload.StatusText = ValidationErrorText;
		SendPullResponse(Payload, SenderAddress);
		return;
	}
	
	Async(EAsyncExecution::Thread, [LocalWeakThis, IncomingMessage = MoveTemp(PullRequestMessage), RemoteAddress = MoveTemp(SenderAddress)]()
	{
		const TSharedPtr<FStormSyncPullMessageService, ESPMode::ThreadSafe> LocalThis = LocalWeakThis.Pin();
		if (!LocalThis)
		{
			return;
		}

		const TSharedPtr<FStormSyncTransportClientEndpoint> LocalEndpoint = LocalThis->GetLocalEndpoint();
		if (!LocalEndpoint.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::HandlePullRequestMessage - Invalid local endpoint.");
			return;
		}

		// Prepare response message payload
		FStormSyncTransportPullResponse Response = LocalThis->CreatePullResponsePayload(IncomingMessage, true);

		UE_LOGF(LogStormSyncClient, 
			Display,
			"FStormSyncPullMessageService::HandlePullRequestMessage - Initializing tcp connection for %ls and message: %ls",
			*RemoteAddress.ToString(),
			*Response.ToString()
		);

		// Case of no modifiers (both remote in sync), early out and respond with payload
		if (Response.Modifiers.IsEmpty())
		{
			const FText ErrorText = LOCTEXT("State_In_Sync", "Detected empty list of modifiers, meaning assets are in the same state.");
			UE_LOGF(LogStormSyncClient, Display, "\t FStormSyncPullMessageService::HandlePullRequestMessage - %ls", *ErrorText.ToString());

			// No modifiers found, meaning both ends are in sync
			Response.Status = EStormSyncResponseResult::Success;
			Response.StatusText = ErrorText;

			LocalThis->SendPullResponse(Response, RemoteAddress);
			return;
		}

		TSharedPtr<FStormSyncTransportClientSocket> Connection;

		// Delegate to catch up any errors during pak buffer creation that happens in a background thread
		const FOnStormSyncSendBufferCallback DoneDelegate = FOnStormSyncSendBufferCallback::CreateLambda([Connection, IncomingMessage, LocalThis, RemoteAddress](const TSharedPtr<FStormSyncSendingBufferPayload>& Payload)
		{
			check(Payload.IsValid());
			check(LocalThis.IsValid());
			
			// If text is non empty, indicates an error happened
			if (!Payload->bSuccess)
			{
				UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::HandlePullRequestMessage - %ls", *Payload->ErrorText.ToString());

				// Got an error, send it back to remote requester
				FStormSyncTransportPullResponse ErrorResponse = LocalThis->CreatePullResponsePayload(IncomingMessage);
				ErrorResponse.Status = EStormSyncResponseResult::Error;
				ErrorResponse.StatusText = Payload->ErrorText;
				LocalThis->SendPullResponse(ErrorResponse, RemoteAddress);
			}
		});
		
		LocalEndpoint->StartSendingBuffer(Response, Connection, DoneDelegate);
		if (!Connection.IsValid())
		{
			const FText ErrorText = LOCTEXT("Error_Active_Connection", "Returned active connection is invalid from endpoint StartSendingBuffer()");
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::HandlePullRequestMessage - %ls", *ErrorText.ToString());

			// Something went wrong with the tcp connection, send it back as an error to remote requester
			FStormSyncTransportPullResponse ErrorResponse = LocalThis->CreatePullResponsePayload(IncomingMessage);
			ErrorResponse.Status = EStormSyncResponseResult::Error;
			ErrorResponse.StatusText = ErrorText;
			LocalThis->SendPullResponse(ErrorResponse, RemoteAddress);
			return;
		}
		
		// Bind to transfer complete delegate, when tcp transfer is done
		Connection->OnTransferComplete().BindThreadSafeSP(LocalThis.Get(), &FStormSyncPullMessageService::HandleTransferComplete, Response, RemoteAddress);	
	});
}

void FStormSyncPullMessageService::HandlePullResponseMessage(const FStormSyncTransportPullResponse& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::HandlePullResponseMessage - Sender: %ls, Message: %ls", *InMessageContext->GetSender().ToString(), *InMessage.ToString());

	FScopeLock ScopeLock(&HandlersCriticalSection);
	const FOnStormSyncPullComplete* ResponseHandler = PendingResponseHandlers.Find(InMessage.MessageId);
	if (ResponseHandler && ResponseHandler->IsBound())
	{
		ResponseHandler->Execute(MakeShared<FStormSyncTransportPullResponse>(InMessage));
		PendingResponseHandlers.Remove(InMessage.MessageId);
	}
}

void FStormSyncPullMessageService::HandleTransferComplete(const FStormSyncTransportPullResponse InResponseMessage, const FMessageAddress InRemoteAddress) const
{
	FStormSyncTransportPullResponse Response = InResponseMessage;
	Response.Status = EStormSyncResponseResult::Success;
	Response.StatusText = FText::Format(LOCTEXT("Pull_Success_TransferComplete", "Successfully pulled {0} package names"), FText::AsNumber(InResponseMessage.PackageNames.Num()));
	
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::HandleTransferComplete - TCP transfer done for message with ID: %ls (Remote: %ls)", *Response.MessageId.ToString(), *InRemoteAddress.ToString());
	SendPullResponse(Response, InRemoteAddress);
}

void FStormSyncPullMessageService::SendPullResponse(const FStormSyncTransportPullResponse& InResponsePayload, const FMessageAddress& InRemoteAddress) const
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::SendPullResponse - Sending to remote %ls payload: %ls", *InRemoteAddress.ToString(), *InResponsePayload.ToString());

	const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = GetMessageEndpoint();
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::SendPullResponse - Unable to get client message endpoint (trying sending to remote: %ls)", *InRemoteAddress.ToString());
		return;
	}

	UE_LOGF(LogStormSyncClient, 
		Display,
		"FStormSyncPullMessageService::SendPullResponse - Sending Pull Response with ID: %ls (Remote: %ls) - Message: %ls",
		*InResponsePayload.MessageId.ToString(),
		*InRemoteAddress.ToString(),
		*InResponsePayload.ToString()
	);

	TUniquePtr<FStormSyncTransportPullResponse> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportPullResponse>(InResponsePayload));
	if (!Message.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncPullMessageService::SendPullResponse - Pull response message is invalid");
		return;
	}

	MessageEndpoint->Send(Message.Release(), InRemoteAddress);
}

bool FStormSyncPullMessageService::ValidateIncomingPullMessage(const FStormSyncTransportPullRequest& InMessage, FText& OutErrorText) const
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::ValidateIncomingPullMessage - Validate incoming pull message: %ls", *InMessage.ToString());
	
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPullMessageService::ValidateIncomingPullMessage - Checking for missing package names on disk ... (%ls)", *InMessage.ToString());

	TArray<FName> PackageNames = InMessage.PackageNames;
	TArray<FString> MissingPackageNames;

	for (const FName& PackageName : PackageNames)
	{
		if (!FPackageName::DoesPackageExist(PackageName.ToString()))
		{
			MissingPackageNames.AddUnique(PackageName.ToString());
		}
	}

	if (!MissingPackageNames.IsEmpty())
	{
		if (MissingPackageNames.Num() == 1)
		{
			check(MissingPackageNames.IsValidIndex(0));
			const FString PackageName = MissingPackageNames[0];
			OutErrorText = FText::Format(LOCTEXT("Missing_Package_OnDisk", "Requesting a pull on package \"{0}\" which does not exist on disk"), FText::FromString(PackageName));
		}
		else
		{
			const FString DisplayPackageNames = FString::Join(MissingPackageNames, TEXT(", "));
			OutErrorText = FText::Format(LOCTEXT("Missing_Packages_OnDisk", "Requesting a pull on packages \"{0}\" which do not exist on disk"), FText::FromString(DisplayPackageNames));
		}
		
		return false;
	}

	return true;
}

FStormSyncTransportPullResponse FStormSyncPullMessageService::CreatePullResponsePayload(const FStormSyncTransportPullRequest& InPullMessage, const bool bInWithModifiers)
{
	FStormSyncTransportPullResponse Payload = FStormSyncTransportPullResponse(InPullMessage.MessageId);

	Payload.PackageNames = InPullMessage.PackageNames;
	Payload.PackageDescriptor = InPullMessage.PackageDescriptor;

	// Forward host network information from incoming message to outgoing sync response
	Payload.HostName = InPullMessage.HostName;
	Payload.HostAddress = InPullMessage.HostAddress;
	Payload.HostAdapterAddresses = InPullMessage.HostAdapterAddresses;

	if (bInWithModifiers)
	{
		// Perform diff here ...
		Payload.Modifiers = FStormSyncCoreUtils::GetSyncFileModifiers(InPullMessage.PackageNames, InPullMessage.PackageDescriptor.Dependencies);
	}

	return Payload;
}

FStormSyncTransportPullResponse FStormSyncPullMessageService::CreatePullResponsePayload(const TArray<FName>& InPackageNames, const FStormSyncPackageDescriptor& InPackageDescriptor)
{
	FStormSyncTransportPullResponse Payload = FStormSyncTransportPullResponse(FGuid::NewGuid());
	Payload.PackageNames = InPackageNames;
	Payload.PackageDescriptor = InPackageDescriptor;
	
	// Fill in payload with network info for this device
	Payload.HostName = FStormSyncTransportNetworkUtils::GetServerName();
	Payload.HostAddress = FStormSyncTransportNetworkUtils::GetCurrentTcpServerEndpointAddress();
	Payload.HostAdapterAddresses = FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses();
	
	return Payload;
}

TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> FStormSyncPullMessageService::GetMessageEndpoint() const
{
	if (const TSharedPtr<IStormSyncTransportLocalEndpoint> LocalEndpoint = GetLocalEndpoint())
	{
		return LocalEndpoint->GetMessageEndpoint();	
	}
	
	return nullptr;
}

TSharedPtr<FStormSyncTransportClientEndpoint> FStormSyncPullMessageService::GetLocalEndpoint() const
{
	return StaticCastSharedPtr<FStormSyncTransportClientEndpoint>(LocalEndpointWeak.Pin());
}

#undef LOCTEXT_NAMESPACE