// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportClientLocalEndpoint.h"
#include "IStormSyncTransportMessageService.h"
#include "MessageEndpoint.h"

class FStormSyncTransportClientEndpoint;
struct FMessageEndpointBuilder;
class IMessageContext;

/**
 * Message service handler for pull related messages
 *
 * This service handles all "pull" related messages and message bus / tcp transaction.
 *
 * Example use case: Client 1 wants to pull some assets from Client 2
 *
 * 1. Client 1 sends a PullRequest message (FStormSyncTransportPullRequest, via RequestPullPackages())
 * 2. Client 2 receives PullRequest message (via HandlePullRequestMessage())
 *	2.1 Client 2 compares the list of file dependency states in incoming message against local file states
 *	2.2 Client 2 prepares a response with a list of "modifiers"
 *	2.3 Client 2 opens a tcp connection and sends a buffer with all the files with mismatched state
 *	2.4 Client 2, on tcp transfer completion, sends a final PullResponse message to Client 1
 * 3. Client 1 receives the PullResponse message from Client 2, and invokes the stored delegate in phase 1. from the GUID that was passed back to back from message to message
 */
class FStormSyncPullMessageService : public IStormSyncTransportMessageService
{
public:
	explicit FStormSyncPullMessageService(TSharedPtr<IStormSyncTransportLocalEndpoint, ESPMode::ThreadSafe> InLocalEndpoint);
	virtual ~FStormSyncPullMessageService();

	//~ Begin IStormSyncTransportMessageService interface
	virtual void InitializeMessageEndpoint(FMessageEndpointBuilder& InEndpointBuilder) override;
	//~ End IStormSyncTransportMessageService interface

	/** Sends a pull synchronization request to the passed in remote address, and waits for the remote to respond triggering the completion delegate */
	void RequestPullPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPullComplete& InDoneDelegate);
	
	/** Abort a request using the GUID previously obtained via a call to RequestPullPackages */
	void AbortPullRequest(const FGuid& InPullRequestId);
private:
	/** Holds the client local endpoint. */
	TWeakPtr<IStormSyncTransportLocalEndpoint> LocalEndpointWeak;
	
	/** Critical section preventing multiple threads from adding / removing handlers simultaneously */
	FCriticalSection HandlersCriticalSection;
	
	/** Handlers awaiting their resultant message bus response */
	TMap<FGuid, FOnStormSyncPullComplete> PendingResponseHandlers;
	
	/** Helper to register a completion delegate protected by a critical section */
	void AddResponseHandler(const FGuid& InId, const FOnStormSyncPullComplete& InDelegate);

	/** Callback handler to receive FStormSyncTransportPushRequest messages */
	void HandlePullRequestMessage(const FStormSyncTransportPullRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);

	/** Callback handler to receive FStormSyncTransportPullResponse messages */
	void HandlePullResponseMessage(const FStormSyncTransportPullResponse& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);

	/** Callback handler when sending tcp buffer to handle transfer completion */
	void HandleTransferComplete(const FStormSyncTransportPullResponse InResponseMessage, const FMessageAddress InRemoteAddress) const;

	/**
	 * Helper to send a PullResponse to the specified remote address recipient.
	 * 
	 * If the response message is tagged with the proper GUID, remote will invoke the stored delegate (if any) with the response.
	 */
	void SendPullResponse(const FStormSyncTransportPullResponse& InResponsePayload, const FMessageAddress& InRemoteAddress) const;

	/**
	 * Prior to preparing a buffer to send, this method can be used to validate an incoming message and return an error in case of:
	 *
	 * - Missing package names on disk (remote wants to pull an asset that doesn't exist on this editor instance)
	 */
	bool ValidateIncomingPullMessage(const FStormSyncTransportPullRequest& InMessage, FText& OutErrorText) const;

	/**
	 * Helper to create a pull response payload and initialize its field based on incoming pull message
	 *
	 * @param InPullMessage Incoming pull message to use to initialize field of payload
	 * @param bInWithModifiers When set to true, payload modifiers will be filled (careful, this operation might
	 * be slow and may be better to be called within an async task)
	 */
	static FStormSyncTransportPullResponse CreatePullResponsePayload(const FStormSyncTransportPullRequest& InPullMessage, const bool bInWithModifiers = false);

	/** Static helper to create a new response payload, in case of an early error during the request phase */
	static FStormSyncTransportPullResponse CreatePullResponsePayload(const TArray<FName>& InPackageNames, const FStormSyncPackageDescriptor& InPackageDescriptor);

	// TODO: Past this point, all helpers etc. might make sense to be contained within a MessageServiceBase class

	/** Returns message endpoint associated with local endpoint */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const;

	/** Returns local endpoint from locally cached WeakPtr */
	TSharedPtr<FStormSyncTransportClientEndpoint> GetLocalEndpoint() const;
};
