// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncPushMessageService.h"

#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncTransportClientEndpoint.h"
#include "StormSyncTransportClientLog.h"
#include "StormSyncTransportNetworkUtils.h"
#include "Socket/StormSyncTransportClientSocket.h"

#define LOCTEXT_NAMESPACE "StormSyncPushMessageService"

FStormSyncPushMessageService::FStormSyncPushMessageService(const TSharedPtr<IStormSyncTransportLocalEndpoint, ESPMode::ThreadSafe> InLocalEndpoint)
	: LocalEndpointWeak(InLocalEndpoint)
{
}

FStormSyncPushMessageService::~FStormSyncPushMessageService()
{
	PendingResponseHandlers.Empty();
}

void FStormSyncPushMessageService::InitializeMessageEndpoint(FMessageEndpointBuilder& InEndpointBuilder)
{
	InEndpointBuilder
		.Handling<FStormSyncTransportPushRequest>(this, &FStormSyncPushMessageService::HandlePushRequestMessage)
		.Handling<FStormSyncTransportPushResponse>(this, &FStormSyncPushMessageService::HandlePushResponseMessage);
}

void FStormSyncPushMessageService::RequestPushPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPushComplete& InDoneDelegate)
{
	FMessageAddress MessageAddress = InRemoteAddress;
	TArray<FName> LocalPackageNames = InPackageNames;
	FStormSyncPackageDescriptor LocalPackageDescriptor = InPackageDescriptor;
	
	TWeakPtr<FStormSyncPushMessageService, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
	
	FStormSyncCoreUtils::GetAvaFileDependenciesAsync(InPackageNames)
		.Then([LocalWeakThis, PackageNames = MoveTemp(LocalPackageNames), PackageDescriptor = MoveTemp(LocalPackageDescriptor), Recipient = MoveTemp(MessageAddress), InDoneDelegate](const TFuture<TArray<FStormSyncFileDependency>> Result)
		{
			const TSharedPtr<FStormSyncPushMessageService, ESPMode::ThreadSafe> LocalThis = LocalWeakThis.Pin();
			if (!LocalThis)
			{
				return;
			}
			
			const TArray<FStormSyncFileDependency> FileDependencies = Result.Get();
			
			// Validate we have at least some file dependencies that were resolved
			if (FileDependencies.IsEmpty())
			{
				const FText ErrorText = LOCTEXT("FileDependencies_Error", "Error before pushing: File dependencies array is empty, something went wrong");
				UE_LOGF(LogStormSyncClient, Error, "FStormSyncPushMessageService::RequestPushPackages - %ls", *ErrorText.ToString());
				
				FStormSyncTransportPushResponse Payload = LocalThis->CreatePushResponsePayload(PackageNames, PackageDescriptor);
				Payload.Status = EStormSyncResponseResult::Error;
				Payload.StatusText = ErrorText;

				InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncTransportPushResponse>(Payload));
				return;
			}

			const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = LocalThis->GetMessageEndpoint();
			if (!MessageEndpoint.IsValid())
			{
				const FText ErrorText = LOCTEXT("InvalidMessageEndpoint_Error", "Error before pushing: Unable to get message endpoint");
				UE_LOGF(LogStormSyncClient, Error, "FStormSyncPushMessageService::RequestPushPackages - %ls", *ErrorText.ToString());

				FStormSyncTransportPushResponse Payload = LocalThis->CreatePushResponsePayload(PackageNames, PackageDescriptor);
				Payload.Status = EStormSyncResponseResult::Error;
				Payload.StatusText = ErrorText;
				InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncTransportPushResponse>(Payload));
				return;
			}

			// Build sync request message that is going to be sent over the network for a specific recipient
			TUniquePtr<FStormSyncTransportPushRequest> PushRequestMessage(FMessageEndpoint::MakeMessage<FStormSyncTransportPushRequest>(PackageNames, PackageDescriptor));
			if (!PushRequestMessage.IsValid())
			{
				const FText ErrorText = LOCTEXT("InvalidPullRequestMessage_Error", "Error before pushing: Push request message is invalid");
				UE_LOGF(LogStormSyncClient, Error, "FStormSyncPushMessageService::RequestPushPackages - %ls", *ErrorText.ToString());

				FStormSyncTransportPushResponse Payload = LocalThis->CreatePushResponsePayload(PackageNames, PackageDescriptor);
				Payload.Status = EStormSyncResponseResult::Error;
				Payload.StatusText = ErrorText;
				InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncTransportPushResponse>(Payload));
				return;
			}
			
			// Store completion callback for later invocation when receiving back a status response
			LocalThis->AddResponseHandler(PushRequestMessage->MessageId, InDoneDelegate);

			PushRequestMessage->PackageDescriptor.Dependencies = FileDependencies;

			UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::RequestPushPackages - FileDependencies: %d", FileDependencies.Num());
			UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::RequestPushPackages - Message: %ls", *PushRequestMessage->ToString());
			UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::RequestPushPackages - Syncing package descriptor %ls", *PushRequestMessage->PackageDescriptor.ToString());

			MessageEndpoint->Send(PushRequestMessage.Release(), Recipient);
		});
}

void FStormSyncPushMessageService::AbortPushRequest(const FGuid& InPushRequestId)
{
	FScopeLock ScopeLock(&HandlersCriticalSection);
	PendingResponseHandlers.Remove(InPushRequestId);
}

void FStormSyncPushMessageService::AddResponseHandler(const FGuid& InId, const FOnStormSyncPushComplete& InDelegate)
{
	FScopeLock ScopeLock(&HandlersCriticalSection);
	PendingResponseHandlers.Add(InId, InDelegate);
}

void FStormSyncPushMessageService::HandlePushRequestMessage(const FStormSyncTransportPushRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::HandlePushRequestMessage - Received push request Message from %ls (%ls)", *InMessageContext->GetSender().ToString(), *InMessage.ToString());

	FGuid MessageId = InMessage.MessageId;
	TArray<FName> LocalPackageNames = InMessage.PackageNames;
	FStormSyncPackageDescriptor LocalPackageDescriptor = InMessage.PackageDescriptor;
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> LocalMessageEndpoint = GetMessageEndpoint();
	FMessageAddress SenderAddress = InMessageContext->GetSender();

	TWeakPtr<FStormSyncPushMessageService, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));

	Async(EAsyncExecution::Thread, [MessageId, MessageEndpointPtr = MoveTemp(LocalMessageEndpoint), PackageNames = MoveTemp(LocalPackageNames), RemotePackageDescriptor = MoveTemp(LocalPackageDescriptor), RemoteAddress = MoveTemp(SenderAddress)]()
	{
		if (!MessageEndpointPtr.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncPushMessageService::HandlePushRequestMessage - Invalid MessageEndpoint, can't sync response back");
			return;
		}

		// Build sync response message that is going to be sent over the network for a specific recipient
		TUniquePtr<FStormSyncTransportPushResponse> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportPushResponse>(MessageId));
		if (!Message.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncStatusMessageService::RequestStatus - Sync response message is invalid");
			return;
		}

		// Perform diff here ...
		const TArray<FStormSyncFileModifierInfo> FileModifiers = FStormSyncCoreUtils::GetSyncFileModifiers(PackageNames, RemotePackageDescriptor.Dependencies);

		Message->PackageNames = PackageNames;
		Message->PackageDescriptor = RemotePackageDescriptor;
		Message->Modifiers = FileModifiers;

		Message->HostName = FStormSyncTransportNetworkUtils::GetServerName();
		Message->HostAddress = FStormSyncTransportNetworkUtils::GetCurrentTcpServerEndpointAddress();
		Message->HostAdapterAddresses = FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses();

		UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::HandlePushRequestMessage - Sending back to %ls sync response: %ls", *RemoteAddress.ToString(), *Message->ToString());
		if (FileModifiers.IsEmpty())
		{
			UE_LOGF(LogStormSyncClient, Display, "\tSending back empty list of modifiers, meaning pak are synced.");
		}

		MessageEndpointPtr->Send(Message.Release(), RemoteAddress);
	});
}

void FStormSyncPushMessageService::HandlePushResponseMessage(const FStormSyncTransportPushResponse& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	const FMessageAddress Sender = InMessageContext->GetSender();

	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::HandlePushResponseMessage - Received sync response from %ls - Message: %ls", *Sender.ToString(), *InMessage.ToString());

	if (InMessage.Modifiers.IsEmpty())
	{
		// Remote sent back empty list of modifiers, meaning assets are synced. Do nothing.
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::HandlePushResponseMessage - Remote sent back empty list of modifiers, meaning pak are synced. Do nothing.");
		
		const FText ErrorText = LOCTEXT("State_In_Sync", "Detected empty list of modifiers, meaning assets are in the same state.");
		UE_LOGF(LogStormSyncClient, Display, "\t FStormSyncPushMessageService::HandlePushResponseMessage - %ls", *ErrorText.ToString());
		
		// But notify caller about completion
		const TSharedPtr<FStormSyncTransportPushResponse> ErrorResponse = MakeShared<FStormSyncTransportPushResponse>(InMessage);

		// No modifiers found, meaning both ends are in sync
		ErrorResponse->Status = EStormSyncResponseResult::Success;
		ErrorResponse->StatusText = ErrorText;

		InvokePendingResponseHandler(ErrorResponse);
		return;
	}

	const TSharedPtr<FStormSyncTransportClientEndpoint> LocalEndpoint = GetLocalEndpoint();
	if (!LocalEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncPushMessageService::HandlePushResponseMessage - Invalid local endpoint.");
		return;	
	}
	
	TSharedPtr<FStormSyncTransportClientSocket> Connection;
	TWeakPtr<FStormSyncPushMessageService, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
	FStormSyncTransportPushResponse PushResponseMessage = InMessage;
	
	// Delegate to catch up any errors during pak buffer creation that happens in a background thread
	const FOnStormSyncSendBufferCallback DoneDelegate = FOnStormSyncSendBufferCallback::CreateLambda([LocalWeakThis, IncomingMessage = MoveTemp(PushResponseMessage)](const TSharedPtr<FStormSyncSendingBufferPayload>& Payload)
	{
		const TSharedPtr<FStormSyncPushMessageService, ESPMode::ThreadSafe> LocalThis = LocalWeakThis.Pin();
		check(Payload.IsValid());
		check(LocalThis.IsValid());
			
		// If text is non empty, indicates an error happened
		if (!Payload->bSuccess)
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncPushMessageService::HandlePushResponseMessage - %ls", *Payload->ErrorText.ToString());
			
			const TSharedPtr<FStormSyncTransportPushResponse> ErrorResponse = MakeShared<FStormSyncTransportPushResponse>(IncomingMessage);

			// Got an error, invoke any stored delegate matching the message ID back to the caller
			ErrorResponse->Status = EStormSyncResponseResult::Error;
			ErrorResponse->StatusText = Payload->ErrorText;
			LocalThis->InvokePendingResponseHandler(ErrorResponse);
		}
	});
	
	LocalEndpoint->StartSendingBuffer(InMessage, Connection, DoneDelegate);
	if (!Connection.IsValid())
	{
		const FText ErrorText = LOCTEXT("Error_Active_Connection", "Returned active connection is invalid from endpoint StartSendingBuffer()");
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncPushMessageService::HandlePushResponseMessage - %ls", *ErrorText.ToString());

		// Something went wrong with the tcp connection, send it back as an error to remote requester
		const TSharedPtr<FStormSyncTransportPushResponse> ErrorResponse = MakeShared<FStormSyncTransportPushResponse>(InMessage);
		
		ErrorResponse->Status = EStormSyncResponseResult::Error;
		ErrorResponse->StatusText = ErrorText;
		InvokePendingResponseHandler(ErrorResponse);
		return;
	}

	// Bind to transfer complete delegate, when tcp transfer is done
	Connection->OnTransferComplete().BindThreadSafeSP(this, &FStormSyncPushMessageService::HandleTransferComplete, InMessage);
}

void FStormSyncPushMessageService::HandleTransferComplete(const FStormSyncTransportPushResponse InResponseMessage)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::HandleTransferComplete - TCP transfer done for message with ID: %ls", *InResponseMessage.MessageId.ToString());
	
	const TSharedPtr<FStormSyncTransportPushResponse> Response =  MakeShared<FStormSyncTransportPushResponse>(InResponseMessage);
	check(Response.IsValid());
	
	Response->Status = EStormSyncResponseResult::Success;
	Response->StatusText = FText::Format(LOCTEXT("Pull_Success_TransferComplete", "Successfully pushed {0} package names"), FText::AsNumber(InResponseMessage.PackageNames.Num()));
	InvokePendingResponseHandler(Response);
}

void FStormSyncPushMessageService::InvokePendingResponseHandler(const TSharedPtr<FStormSyncTransportPushResponse, ESPMode::ThreadSafe>& InResponseMessage)
{
	check(InResponseMessage.IsValid());
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncPushMessageService::InvokePendingResponseHandler - Trying to invoke callback (if any) for message with ID: %ls", *InResponseMessage->ToString());
	
	FScopeLock ScopeLock(&HandlersCriticalSection);
	const FOnStormSyncPushComplete* ResponseHandler = PendingResponseHandlers.Find(InResponseMessage->MessageId);
	if (ResponseHandler && ResponseHandler->IsBound())
	{
		ResponseHandler->Execute(InResponseMessage);
		PendingResponseHandlers.Remove(InResponseMessage->MessageId);
	}
}

FStormSyncTransportPushResponse FStormSyncPushMessageService::CreatePushResponsePayload(const TArray<FName>& InPackageNames, const FStormSyncPackageDescriptor& InPackageDescriptor)
{
	FStormSyncTransportPushResponse Payload = FStormSyncTransportPushResponse(FGuid::NewGuid());
	Payload.PackageNames = InPackageNames;
	Payload.PackageDescriptor = InPackageDescriptor;
	
	// Fill in payload with network info for this device
	Payload.HostName = FStormSyncTransportNetworkUtils::GetServerName();
	Payload.HostAddress = FStormSyncTransportNetworkUtils::GetCurrentTcpServerEndpointAddress();
	Payload.HostAdapterAddresses = FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses();
	
	return Payload;
}

TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> FStormSyncPushMessageService::GetMessageEndpoint() const
{
	if (const TSharedPtr<IStormSyncTransportLocalEndpoint> LocalEndpoint = GetLocalEndpoint())
	{
		return LocalEndpoint->GetMessageEndpoint();
	}

	return nullptr;
}

TSharedPtr<FStormSyncTransportClientEndpoint> FStormSyncPushMessageService::GetLocalEndpoint() const
{
	return StaticCastSharedPtr<FStormSyncTransportClientEndpoint>(LocalEndpointWeak.Pin());
}

#undef LOCTEXT_NAMESPACE