// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportClientLocalEndpoint.h"
#include "IStormSyncTransportMessageService.h"

class FStormSyncTransportClientEndpoint;
class IMessageContext;
struct FMessageEndpointBuilder;

/** Message service handler for push related messages */
class FStormSyncPushMessageService : public IStormSyncTransportMessageService
{
public:
	explicit FStormSyncPushMessageService(TSharedPtr<IStormSyncTransportLocalEndpoint, ESPMode::ThreadSafe> InLocalEndpoint);
	virtual ~FStormSyncPushMessageService() override;

	//~ Begin IStormSyncTransportMessageService interface
	virtual void InitializeMessageEndpoint(FMessageEndpointBuilder& InEndpointBuilder) override;
	//~ End IStormSyncTransportMessageService interface

	/** Sends a push synchronization request to the passed in remote address, and waits for the remote to respond triggering the completion delegate */
	void RequestPushPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPushComplete& InDoneDelegate);

	/** Abort a request using the GUID previously obtained via a call to RequestPushPackages */
	void AbortPushRequest(const FGuid& InPushRequestId);

private:
	/** Holds the client local endpoint. */
	TWeakPtr<IStormSyncTransportLocalEndpoint> LocalEndpointWeak;
	
	/** Critical section preventing multiple threads from adding / removing handlers simultaneously */
	FCriticalSection HandlersCriticalSection;
	
	/** Handlers awaiting their resultant message bus response */
	TMap<FGuid, FOnStormSyncPushComplete> PendingResponseHandlers;
	
	/** Helper to register a completion delegate protected by a critical section */
	void AddResponseHandler(const FGuid& InId, const FOnStormSyncPushComplete& InDelegate);

	/** Callback handler to receive FStormSyncTransportPushRequest messages */
	void HandlePushRequestMessage(const FStormSyncTransportPushRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);

	/** Callback handler to receive FStormSyncTransportPushResponse messages */
	void HandlePushResponseMessage(const FStormSyncTransportPushResponse& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);
	
	/** Callback handler when sending tcp buffer to handle transfer completion */
	void HandleTransferComplete(const FStormSyncTransportPushResponse InResponseMessage);

	/**
	 * Helper to invoke a pending response handler based on the provided message (and ID)
	 *
	 * If a previously stored delegate is matching the message ID in PendingResponseHandlers, it will be invoked and cleaned up.
	 */
	void InvokePendingResponseHandler(const TSharedPtr<FStormSyncTransportPushResponse, ESPMode::ThreadSafe>& InResponseMessage);
	
	/** Static helper to create a new response payload, in case of an early error during the request phase */
	static FStormSyncTransportPushResponse CreatePushResponsePayload(const TArray<FName>& InPackageNames, const FStormSyncPackageDescriptor& InPackageDescriptor);

	/** Returns message endpoint associated with local endpoint */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const;

	/** Returns local endpoint from locally cached WeakPtr */
	TSharedPtr<FStormSyncTransportClientEndpoint> GetLocalEndpoint() const;
};
