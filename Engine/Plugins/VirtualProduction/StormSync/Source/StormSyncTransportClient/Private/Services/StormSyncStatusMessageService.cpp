// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncStatusMessageService.h"

#include "IStormSyncTransportClientModule.h"
#include "IStormSyncTransportLocalEndpoint.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncTransportClientLog.h"

FStormSyncStatusMessageService::FStormSyncStatusMessageService(const TSharedPtr<IStormSyncTransportLocalEndpoint, ESPMode::ThreadSafe> InLocalEndpoint)
	: LocalEndpointWeak(InLocalEndpoint)
{
}

FStormSyncStatusMessageService::~FStormSyncStatusMessageService()
{
	PendingStatusResponseHandlers.Empty();
}

void FStormSyncStatusMessageService::InitializeMessageEndpoint(FMessageEndpointBuilder& InEndpointBuilder)
{
	InEndpointBuilder
		.Handling<FStormSyncTransportStatusPing>(this, &FStormSyncStatusMessageService::HandleStatusPing)
		.Handling<FStormSyncTransportStatusRequest>(this, &FStormSyncStatusMessageService::HandleStatusRequest)
		.Handling<FStormSyncTransportStatusResponse>(this, &FStormSyncStatusMessageService::HandleStatusResponse);
}

void FStormSyncStatusMessageService::RequestStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& DoneDelegate)
{
	TArray<FName> LocalPackageNames = InPackageNames;
	FMessageAddress MessageAddress = InRemoteAddress;

	TWeakPtr<FStormSyncStatusMessageService, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
	
	FStormSyncCoreUtils::GetAvaFileDependenciesAsync(InPackageNames)
	.Then([LocalWeakThis, PackageNames = MoveTemp(LocalPackageNames), RemoteAddress = MoveTemp(MessageAddress), DoneDelegate](const TFuture<TArray<FStormSyncFileDependency>> Result)
	{
		const TArray<FStormSyncFileDependency> FileDependencies = Result.Get();
		if (FileDependencies.IsEmpty())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncStatusMessageService::RequestStatus - Async FileDependencies is empty, something went wrong");
			return;
		}

		const TSharedPtr<FStormSyncStatusMessageService, ESPMode::ThreadSafe> LocalThis = LocalWeakThis.Pin();
		if (!LocalThis)
		{
			return;
		}

		const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = LocalThis->GetMessageEndpoint();
		if (!MessageEndpoint.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncStatusMessageService::RequestStatus - Unable to get message endpoint");
			return;
		}

		// Build sync request message that is going to be sent over the network for a specific recipient
		TUniquePtr<FStormSyncTransportStatusRequest> RequestStatusMessage(FMessageEndpoint::MakeMessage<FStormSyncTransportStatusRequest>(PackageNames, FileDependencies));
		if (!RequestStatusMessage.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncStatusMessageService::RequestStatus - Push request message is invalid");
			return;
		}

		// Store completion callback for later invocation when receiving back a status response
		LocalThis->AddResponseHandler(RequestStatusMessage->MessageId, DoneDelegate);

		UE_LOGF(LogStormSyncClient, Display, "FStormSyncStatusMessageService::RequestStatus - FileDependencies: %d", FileDependencies.Num());
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncStatusMessageService::RequestStatus - Message: %ls", *RequestStatusMessage->ToString());

		MessageEndpoint->Send(RequestStatusMessage.Release(), RemoteAddress);
	});
}

void FStormSyncStatusMessageService::AbortStatusRequest(const FGuid& InStatusRequestId)
{
	FScopeLock ScopeLock(&HandlersCriticalSection);
	PendingStatusResponseHandlers.Remove(InStatusRequestId);
}

void FStormSyncStatusMessageService::AddResponseHandler(const FGuid& InId, const FOnStormSyncRequestStatusComplete& InDelegate)
{
	FScopeLock ScopeLock(&HandlersCriticalSection);
	PendingStatusResponseHandlers.Add(InId, InDelegate);
}

void FStormSyncStatusMessageService::HandleStatusPing(const FStormSyncTransportStatusPing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
{
	const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = GetMessageEndpoint();
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncStatusMessageService::HandleStatusPing - Unable to get client endpoint");
		return;
	}

	UE_LOGF(LogStormSyncClient, Display, "FStormSyncStatusMessageService::HandleStatusPing - Sender: %ls", *MessageContext->GetSender().ToString());

	// Send back pong message so that this recipient knows about this editor instance
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncStatusMessageService::HandleStatusPing - Send pong Message back to %ls...", *MessageContext->GetSender().ToString());
	FStormSyncTransportStatusPong* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportStatusPong>();
	MessageEndpoint->Send(Message, MessageContext->GetSender());
}

void FStormSyncStatusMessageService::HandleStatusRequest(const FStormSyncTransportStatusRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
{
	FMessageAddress Sender = MessageContext->GetSender();

	UE_LOGF(LogStormSyncClient, Display,
		"FStormSyncStatusMessageService::HandleStatusRequest - Message: %ls, Sender: %ls", *InMessage.ToString(), *Sender.ToString());

	const TWeakPtr<FMessageEndpoint> MessageEndpointWeak = GetMessageEndpoint();

	Async(EAsyncExecution::Thread, [Sender, MessageEndpointWeak, MessageId = InMessage.MessageId, PackageNames = InMessage.PackageNames, Dependencies = InMessage.Dependencies]()
	{
		TUniquePtr<FStormSyncTransportStatusResponse> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportStatusResponse>(MessageId));

		UE_LOGF(LogStormSyncClient, Display, "GetSyncFileModifiers for Packages %ls",
			*FString::JoinBy(PackageNames, TEXT(" ,"), [](const FName& InName){return InName.ToString();}));
	
		// Compute the diff
		Message->Modifiers = FStormSyncCoreUtils::GetSyncFileModifiers(PackageNames, Dependencies);
		Message->bNeedsSynchronization = !Message->Modifiers.IsEmpty();

		UE_LOGF(LogStormSyncClient, Display, "GetSyncFileModifiers ... - Modifiers: %d", Message->Modifiers.Num());
		for (const FStormSyncFileModifierInfo& Modifier : Message->Modifiers)
		{
			UE_LOGF(LogStormSyncClient, Display, "\t Modifier: %ls", *Modifier.ToString());
		}

		const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = MessageEndpointWeak.Pin();
		if (!MessageEndpoint.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncStatusMessageService::HandleStatusRequest - Unable to get client endpoint");
			return;
		}
		
		// Send back result
		MessageEndpoint->Send(Message.Release(), Sender);
	});
}

void FStormSyncStatusMessageService::HandleStatusResponse(const FStormSyncTransportStatusResponse& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncStatusMessageService::HandleStatusResponse - Message: %ls, Sender: %ls (Id: %ls)", *InMessage.ToString(), *MessageContext->GetSender().ToString(), *InMessage.StatusRequestId.ToString());

	TArray<FStormSyncFileModifierInfo> NewModifiers;
	TArray<FStormSyncFileModifierInfo> Modifiers = InMessage.Modifiers;
	for (const FStormSyncFileModifierInfo& Modifier : Modifiers)
	{
		UE_LOGF(LogStormSyncClient, Display, "\t Original Modifier: %ls", *Modifier.ToString());

		// Handle "missing" case, meaning remote is referencing the file but we are not
		// It doesn't mean we don't have the file though, check for file equality here
		if (Modifier.ModifierOperation == EStormSyncModifierOperation::Missing)
		{
			FStormSyncFileDependency FileDependency = FStormSyncCoreUtils::CreateStormSyncFile(Modifier.FileDependency.PackageName);

			// Add it to modifiers if we could not find it locally
			if (!FileDependency.IsValid())
			{
				UE_LOGF(LogStormSyncClient, Display, "\t\t Adding as missing locally, Modifier: %ls", *Modifier.ToString());
				NewModifiers.Add(Modifier);
				continue;
			}

			// If state mismatched add it too
			FStormSyncFileDependency RemoteDependency = Modifier.FileDependency;

			const bool bSameSize = FileDependency.FileSize == RemoteDependency.FileSize;
			const bool bSameHash = FileDependency.FileHash == RemoteDependency.FileHash;

			if (!bSameSize || !bSameHash)
			{
				UE_LOGF(LogStormSyncClient, Display, "\t\t Mismatched state locally, add it as overwrite operation. Modifier: %ls", *Modifier.ToString());

				// Mismatched state, change modifier operation to overwrite
				NewModifiers.Add(FStormSyncFileModifierInfo(EStormSyncModifierOperation::Overwrite, RemoteDependency));
			}
		}
		else
		{
			NewModifiers.Add(Modifier);
		}
	}

	UE_LOGF(LogStormSyncClient, Display, "FStormSyncStatusMessageService::HandleStatusResponse - NewModifiers: %d", NewModifiers.Num());
	for (const FStormSyncFileModifierInfo& Modifier : NewModifiers)
	{
		UE_LOGF(LogStormSyncClient, Display, "\t New Modifier: %ls", *Modifier.ToString());
	}

	{
		FScopeLock ScopeLock(&HandlersCriticalSection);
		const FOnStormSyncRequestStatusComplete* ResponseHandler = PendingStatusResponseHandlers.Find(InMessage.StatusRequestId);
		if (ResponseHandler && ResponseHandler->IsBound())
		{
			ResponseHandler->Execute(MakeShared<FStormSyncTransportStatusResponse>(InMessage));
			PendingStatusResponseHandlers.Remove(InMessage.StatusRequestId);
		}
	}
}

TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> FStormSyncStatusMessageService::GetMessageEndpoint() const
{
	if (const TSharedPtr<IStormSyncTransportLocalEndpoint> LocalEndpoint = GetLocalEndpoint())
	{
		return LocalEndpoint->GetMessageEndpoint();
	}
	
	return nullptr;
}

TSharedPtr<IStormSyncTransportLocalEndpoint> FStormSyncStatusMessageService::GetLocalEndpoint() const
{
	return LocalEndpointWeak.Pin();
}
