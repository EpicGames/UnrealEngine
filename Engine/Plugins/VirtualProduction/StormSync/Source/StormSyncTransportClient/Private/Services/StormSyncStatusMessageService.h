// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportClientLocalEndpoint.h"
#include "IStormSyncTransportMessageService.h"

struct FMessageEndpointBuilder;
class IMessageContext;

/** Message service handler for status related messages */
class FStormSyncStatusMessageService : public IStormSyncTransportMessageService
{
public:
	explicit FStormSyncStatusMessageService(TSharedPtr<IStormSyncTransportLocalEndpoint, ESPMode::ThreadSafe> InLocalEndpoint);
	virtual ~FStormSyncStatusMessageService() override;

	//~ Begin IStormSyncTransportMessageService interface
	virtual void InitializeMessageEndpoint(FMessageEndpointBuilder& InEndpointBuilder) override;
	//~ End IStormSyncTransportMessageService interface

	/** Request a status from the given remote address. The given delegate will be called when the response comes in */
	void RequestStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& DoneDelegate);

	/** Abort a request using the GUID previously obtained via a call to RequestStatus */
	void AbortStatusRequest(const FGuid& InStatusRequestId);

private:
	/** Holds the client local endpoint. */
	TWeakPtr<IStormSyncTransportLocalEndpoint> LocalEndpointWeak;

	/** Handlers awaiting their resultant message bus response */
	TMap<FGuid, FOnStormSyncRequestStatusComplete> PendingStatusResponseHandlers;

	/** Critical section preventing multiple threads from adding / removing handlers simultaneously */
	FCriticalSection HandlersCriticalSection;

	/** Helper to register a completion delegate protected by a critical section */
	void AddResponseHandler(const FGuid& InId, const FOnStormSyncRequestStatusComplete& InDelegate);

	/** TODO: Why do we have receive a message first */
	void HandleStatusPing(const FStormSyncTransportStatusPing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext);

	/** Callback handler to receive status requests */
	void HandleStatusRequest(const FStormSyncTransportStatusRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext);

	/** Callback handler to receive status responses */
	void HandleStatusResponse(const FStormSyncTransportStatusResponse& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext);

	/** Returns message endpoint associated with local endpoint */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const;

	TSharedPtr<IStormSyncTransportLocalEndpoint> GetLocalEndpoint() const;
};
