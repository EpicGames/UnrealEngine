// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportClientSocket.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Common/TcpSocketBuilder.h"
#include "Dom/JsonObject.h"
#include "HAL/RunnableThread.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "StormSyncTransportClientLog.h"

FStormSyncTransportClientSocket::FStormSyncTransportClientSocket(const FIPv4Endpoint& InEndpoint)
	: ConnectionState(State_Connecting)
	, RemoteEndpoint(InEndpoint)
	, Thread(nullptr)
	, Socket(nullptr)
{
	Settings = GetDefault<UStormSyncTransportSettings>();
	ConnectionRetryDelay = Settings->GetConnectionRetryDelay();
}

FStormSyncTransportClientSocket::~FStormSyncTransportClientSocket()
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientSocket::~FStormSyncTransportClientSocket Destructor Closing socket and runnable to '%ls'", *RemoteEndpoint.ToString());
	
	if (Thread != nullptr)
	{
		if (!bStopping)
		{
			bStopping = true;
			Thread->WaitForCompletion();
		}
		delete Thread;
	}

	if (Socket)
	{
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
	}

	if (ConnectionClosedDelegate.IsBound())
	{
		ConnectionClosedDelegate.Unbind();
	}

	if (ConnectionStateChangedDelegate.IsBound())
	{
		ConnectionStateChangedDelegate.Unbind();
	}
}

void FStormSyncTransportClientSocket::StartTransport()
{
	check(Thread == nullptr);
	bStopping = false;
	Thread = FRunnableThread::Create(this, *FString::Printf(TEXT("FStormSyncTransportClientSocket %s"), *RemoteEndpoint.ToString()), 128 * 1024, TPri_Normal);
}

void FStormSyncTransportClientSocket::StopTransport()
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientSocket::StopTransport Closing socket and runnable to '%ls'", *RemoteEndpoint.ToString());
	
	// let the thread shutdown on its own
	if (Thread != nullptr)
	{
		bStopping = true;
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}

	// if there a socket, close it so our peer will get a quick disconnect notification
	if (Socket)
	{
		Socket->Close();
		ConnectionClosedDelegate.ExecuteIfBound(RemoteEndpoint);
	}
}

bool FStormSyncTransportClientSocket::IsTransportRunning() const
{
	return !bStopping && Thread != nullptr;
}

bool FStormSyncTransportClientSocket::Init()
{
	// If we follow FTcpMessageTransportConnection pattern of caller dealing with socket creation / connection
	// return (Socket != nullptr);
	return true;
}

void FStormSyncTransportClientSocket::Stop()
{
	if (Socket)
	{
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientSocket::Stop Closing socket to '%ls'", *RemoteEndpoint.ToString());
		Socket->Close();
		ConnectionClosedDelegate.ExecuteIfBound(RemoteEndpoint);
	}
}

uint32 FStormSyncTransportClientSocket::Run()
{
	UE_LOGF(LogStormSyncClient, Log, "Started Connection to '%ls'", *RemoteEndpoint.ToString());

	double LastTime = FPlatformTime::Seconds();
	constexpr float IdealFrameTime = 1.0f / 30.0f;

	bool bListenerIsRunning = true;

	while (bListenerIsRunning && !bStopping)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double DeltaTime = CurrentTime - LastTime;
		LastTime = CurrentTime;

		if (DeltaTime > 1.2f)
		{
			UE_LOGF(LogStormSyncClient, Warning, "Hitch detected; %.3f seconds since prior tick", DeltaTime);
		}

		FString StopReason;
		bListenerIsRunning = Tick(StopReason);
		
		UE_LOGF(LogStormSyncClient, 
			Verbose,
			"Running ... Connected: %ls (%ls), State: %ls, bListenerIsRunning: %ls %ls",
			IsConnected() ? TEXT("true") : TEXT("false"),
			*RemoteEndpoint.ToString(),
			*GetReadableConnectionState(ConnectionState),
			bListenerIsRunning ? TEXT("true") : TEXT("false"),
			*StopReason
		);

		if (!bListenerIsRunning && GetConnectionState() == State_Closed && ConnectionRetryDelay > 0)
		{
			bListenerIsRunning = TryReconnect();
		}

		if (!bListenerIsRunning)
		{
			UE_LOGF(LogStormSyncClient, Display, "Stopped. Reason: %ls", *StopReason);
		}

		// Throttle main thread main fps by sleeping if we still have time
		FPlatformProcess::Sleep(FMath::Max<float>(0.01f, IdealFrameTime - (FPlatformTime::Seconds() - CurrentTime)));
	}

	return 0;
}

bool FStormSyncTransportClientSocket::Tick(FString& StopReason)
{
	if (!Socket)
	{
		// No socket yet
		return true;
	}
	
	// check if the socket has closed
	{
		int32 BytesRead;
		uint8 Dummy;
		if (!Socket->Recv(&Dummy, 1, BytesRead, ESocketReceiveFlags::Peek))
		{
			const FString ErrorCode = GetSocketReadableErrorCode();
			StopReason = FString::Printf(TEXT("Dummy read failed with code %s. Socket has closed [%s]"), *ErrorCode, *RemoteEndpoint.ToString());
			UE_LOGF(LogStormSyncClient, Verbose, "%ls", *StopReason);

			{
				FScopeLock SendLock(&SendCriticalSection);
				ConnectionState = State_Closed;
				bSending = false;
			}
			ConnectionStateChangedDelegate.ExecuteIfBound();
			
			return false;
		}
	}

	// Block waiting for some data
	if (!Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(SocketWaitForReadTimeSeconds)))
	{
		const bool bConnectionOk = Socket->GetConnectionState() != SCS_ConnectionError;
		if (!bConnectionOk)
		{
			StopReason = FString::Printf(TEXT("Wait for read failed. Socket connection state is errored [%s]"), *RemoteEndpoint.ToString());
		}
		
		return bConnectionOk;
	}

	uint32 BufferSize = 0;
	if (Socket->HasPendingData(BufferSize))
	{
		TArray<uint8> ReceiveBuffer;
		ReceiveBuffer.SetNumUninitialized(BufferSize);

		int32 BytesRead = 0;
		if (!Socket->Recv(ReceiveBuffer.GetData(), ReceiveBuffer.Num(), BytesRead))
		{
			const FString ErrorCode = GetSocketReadableErrorCode();
			StopReason = FString::Printf(TEXT("Read failed with code %s [%s]"), *ErrorCode, *RemoteEndpoint.ToString());
			UE_LOGF(LogStormSyncClient, Verbose, "%ls", *StopReason);
			return false;
		}

		UpdateConnectionState(State_Connected);
		
		if (BytesRead > 0)
		{
			check(BytesRead == ReceiveBuffer.Num());
			TotalBytesReceived += BytesRead;
			
			ParseIncomingBytes(BytesRead, ReceiveBuffer);
		}
		else
		{
			// no data
			return true;
		}
	}

	if (IsEngineExitRequested())
	{
		StopReason = FString::Printf(TEXT("Engine is requesting exit"));
		return false;
	}

	return true;
}

bool FStormSyncTransportClientSocket::TryReconnect()
{
	bool bReconnectPending;

	{
		// Wait for any sending before we close the socket
		FScopeLock SendLock(&SendCriticalSection);

		if (Socket)
		{
			Socket->Close();
			ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
			Socket = nullptr;

			ConnectionClosedDelegate.ExecuteIfBound(RemoteEndpoint);
		}
    
		FPlatformProcess::Sleep(ConnectionRetryDelay);

		bReconnectPending = Connect();
		if (bReconnectPending)
		{
			ConnectionState = State_DisconnectReconnectPending;
		}
	}

	if (bReconnectPending)
	{
		ConnectionStateChangedDelegate.ExecuteIfBound();
	}
	
	return bReconnectPending;
}

bool FStormSyncTransportClientSocket::Connect()
{
	bool bSuccess = false;
	UE_LOGF(LogStormSyncClient, Verbose, "Connection to '%ls', trying...", *RemoteEndpoint.ToString());
    
	Socket = FTcpSocketBuilder(SocketDescription)
		.WithSendBufferSize(SocketBufferSize)
		.WithReceiveBufferSize(SocketBufferSize);

	if (Socket && Socket->Connect(RemoteEndpoint.ToInternetAddr().Get()))
	{
		const ESocketConnectionState SocketConnectionState = Socket->GetConnectionState();
		
		// Check status of socket connection, regardless of Connect return status
		// Connection could be in error state, indicating an issue when connecting to remote.
		// Could be trying to establish connection on wrong subnet or a firewall issue
		if (SocketConnectionState != SCS_ConnectionError)
		{
			bSuccess = true;
			{
				FScopeLock SendLock(&SendCriticalSection);
				ConnectionState = State_Connected;
			}
			ConnectionStateChangedDelegate.ExecuteIfBound();

			UE_LOGF(LogStormSyncClient, 
				Display,
				"Connection to %ls successful. Connection State: %ls",
				*RemoteEndpoint.ToString(),
				*GetSocketReadableConnectionState(SocketConnectionState)
			);
		}
		else
		{
			bSuccess = false;
			
			const FString SocketErrorCode = GetSocketReadableErrorCode();
			UE_LOGF(LogStormSyncClient, 
				Warning,
				"Socket connected to %ls with invalid state: %ls (Last Error Code: %ls)",
				*RemoteEndpoint.ToString(),
				*GetSocketReadableConnectionState(SocketConnectionState),
				*SocketErrorCode
			);
			UE_LOGF(LogStormSyncClient, Warning, "\t- Ensure the connection is on the correct subnet.");
			UE_LOGF(LogStormSyncClient, Warning, "\t- Ensure the firewall settings are not blocking the connection.");
		}
	}

	if (!bSuccess)
	{
		if (Socket)
		{
			ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
			Socket = nullptr;
		}
		return false;
	}

	// Check if the socket is actually opened, server should respond with state packet on connection
	if (Socket)
	{
		// Block waiting for some data
		if (!Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(SocketWaitForReadTimeSeconds)))
		{
			UE_LOGF(LogStormSyncClient, 
				Warning,
				"Connect wait for state packet failed. We were able to connect to %ls but server didn't respond (Last Error Code: %ls)",
				*RemoteEndpoint.ToString(),
				*GetSocketReadableErrorCode()
			);
			return false;
		}

		uint32 PendingDataSize = 0;
		if (Socket->HasPendingData(PendingDataSize))
		{
			TArray<uint8> Buffer;
			Buffer.AddUninitialized(PendingDataSize);
			int32 BytesRead = 0;
			if (!Socket->Recv(Buffer.GetData(), PendingDataSize, BytesRead, ESocketReceiveFlags::None))
			{
				UE_LOGF(LogStormSyncClient, Error, "Error while receiving data via endpoint %ls", *RemoteEndpoint.ToString());
				return false;
			}

			TArray<uint8> MessageBuffer;
			for (int32 i = 0; i < BytesRead; ++i)
			{
				MessageBuffer.Add(Buffer[i]);
				if (Buffer[i] == '\x00')
				{
					const FString Message(UTF8_TO_TCHAR(MessageBuffer.GetData()));
					ParseIncomingMessage(Message, RemoteEndpoint);
					MessageBuffer.Empty();
				}
			}
		}
	}

	return bSuccess;
}

void FStormSyncTransportClientSocket::CloseSocket(const bool bShouldTriggerEvent)
{
	if (Socket)
	{
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientSocket::CloseSocket Closing socket to '%ls'", *RemoteEndpoint.ToString());
		Socket->Close();

		if (bShouldTriggerEvent)
		{
			ConnectionClosedDelegate.ExecuteIfBound(RemoteEndpoint);
		}
	}
}

bool FStormSyncTransportClientSocket::IsConnected() const
{
	return Socket && Socket->GetConnectionState() == SCS_Connected;
}

void FStormSyncTransportClientSocket::SendBuffer(const FStormSyncBuffer& InBuffer)
{
	const int32 BufferSize = InBuffer.Num();

	CurrentBytes = 0;
	TotalBytes = BufferSize;

	const EConnectionState State = GetConnectionState();

	UE_LOGF(LogStormSyncClient, 
		Display,
		"FStormSyncTransportClientSocket::SendBuffer - Sending buffer of size %d to %ls:%d endpoint (State: %ls)",
		BufferSize,
		*RemoteEndpoint.Address.ToString(),
		RemoteEndpoint.Port,
		*GetReadableConnectionState(State)
	);

	// The first 4 bytes are going to be our expected buffer size on the server end
	bSending = Send(InBuffer.GetData(), BufferSize);
}

FStormSyncTransportClientSocket::EConnectionState FStormSyncTransportClientSocket::GetConnectionState() const
{
	return ConnectionState;
}

FString FStormSyncTransportClientSocket::GetReadableConnectionState(const EConnectionState State)
{
	switch (State)
	{
		case State_Connecting: return TEXT("STATE_Connecting");
		case State_Connected: return TEXT("STATE_Connected");
		case State_DisconnectReconnectPending: return TEXT("STATE_DisconnectReconnectPending");
		case State_Closed: return TEXT("STATE_Closed");
		default: return TEXT("Unknown State");
	};
}

bool FStormSyncTransportClientSocket::Send(const uint8* Data, uint32 Size, const bool bPrependSize)
{
	if (!Socket)
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientSocket::Send - Socket is not active. Make sure to call Connect() prior to sending buffers.");
		return false;
	}
	
	if (ConnectionState != State_Connected)
	{
		UE_LOGF(LogStormSyncClient, 
			Warning,
			"FStormSyncTransportClientSocket::Send - Connection State seems to be invalid: %ls (Socket State: %ls, Last Socket Error: %ls)",
			*GetReadableConnectionState(ConnectionState),
			*GetSocketReadableConnectionState(Socket->GetConnectionState()),
			*GetSocketReadableErrorCode()
		);
		
		UE_LOGF(LogStormSyncClient, Warning, "Attempted to send a socket message but no socket connected, ignoring...");
		return false;
	}
	
	FScopeLock SendLock(&SendCriticalSection);
	
	// See if we're writable
	if (!Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(SocketWaitForWriteTimeSeconds)))
	{
		return false;
	}

	TArray<uint8> Buffer;

	if (bPrependSize)
	{
		// Insert size
		Buffer.Append(reinterpret_cast<uint8*>(&Size), sizeof(uint32));
	}

	Buffer.Append(Data, Size);

	UE_LOGF(LogStormSyncClient, Display, "Client: Sending %d bytes (Size: %d) ...", Buffer.Num(), Size);
	if (!BlockingSend(Buffer.GetData(), Buffer.Num()))
	{
		const FString ErrorCode = GetSocketReadableErrorCode();
		UE_LOGF(LogStormSyncClient, Warning, "FStormSyncTransportClientSocket::BlockingSend failed with code %ls", *ErrorCode);
		return false;
	}

	TotalBytesSent += Buffer.Num();

	return true;
}

bool FStormSyncTransportClientSocket::BlockingSend(const uint8* Data, int32 BytesToSend)
{
	while (BytesToSend > 0)
	{
		while (!Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(SocketWaitForWriteTimeSeconds)))
		{
			if (Socket->GetConnectionState() == SCS_ConnectionError)
			{
				return false;
			}
		}

		int32 BytesSent = 0;
		if (!Socket->Send(Data, BytesToSend, BytesSent))
		{
			return false;
		}
		
		BytesToSend -= BytesSent;
		Data += BytesSent;
	}
	
	return true;
}

void FStormSyncTransportClientSocket::ParseIncomingBytes(const int32 InNumBytesRead, const TArray<uint8>& InBytes)
{
	TArray<uint8> MessageBuffer;
	MessageBuffer.Reserve(InNumBytesRead);
	
	for (int32 i = 0; i < InNumBytesRead; ++i)
	{
		MessageBuffer.Add(InBytes[i]);
		if (InBytes[i] == '\x00')
		{
			const FString Message(UTF8_TO_TCHAR(MessageBuffer.GetData()));
			ParseIncomingMessage(Message, RemoteEndpoint);
			MessageBuffer.Reset();
		}
	}
}

bool FStormSyncTransportClientSocket::ParseIncomingMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint)
{
	UE_LOGF(LogStormSyncClient, VeryVerbose, "FStormSyncTransportClientSocket::ParseIncomingMessage from %ls - Message: %ls", *InEndpoint.ToString(), *InMessage);

	const TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(InMessage);
	TSharedPtr<FJsonObject> JsonData;

	if (!FJsonSerializer::Deserialize(Reader, JsonData))
	{
		return false;
	}

	const TSharedPtr<FJsonValue> CommandField = JsonData->TryGetField(TEXT("command"));
	if (!CommandField.IsValid())
	{
		return false;
	}
	
	const FString CommandName = CommandField->AsString().ToLower();

	// Handle "size" packets
	if (CommandName == TEXT("size"))
	{
		const FString FieldName = TEXT("receivedBytes");
		double ReceivedBytes;
		if (!JsonData->TryGetNumberField(FieldName, ReceivedBytes))
		{
			UE_LOGF(LogStormSyncClient, 
				Error,
				"FStormSyncTransportClientSocket::ParseIncomingMessage \"size\" command missing required field \"%ls\" or not able to convert to number",
				*FieldName
			);
			return false;
		}

		// Ensure delegates run on game thread
		TWeakPtr<FStormSyncTransportClientSocket, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
		AsyncTask(ENamedThreads::GameThread, [LocalWeakThis, ReceivedBytes]()
		{
			if (const TSharedPtr<FStormSyncTransportClientSocket, ESPMode::ThreadSafe> StrongThis = LocalWeakThis.Pin(); StrongThis.IsValid())
			{
				StrongThis->ReceivedSizeDelegate.ExecuteIfBound(ReceivedBytes);
			}
		});
	}
	// Handle "transfer_complete" packets
	else if (CommandName == TEXT("transfer_complete"))
	{
		// Ensure delegates run on game thread
		TWeakPtr<FStormSyncTransportClientSocket, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
		AsyncTask(ENamedThreads::GameThread, [LocalWeakThis]()
		{
			if (const TSharedPtr<FStormSyncTransportClientSocket, ESPMode::ThreadSafe> StrongThis = LocalWeakThis.Pin(); StrongThis.IsValid())
			{
				StrongThis->TransferCompleteDelegate.ExecuteIfBound();
			}
		});
	}
	
	return true;
}

void FStormSyncTransportClientSocket::UpdateConnectionState(const EConnectionState InNewState)
{
	const EConnectionState PrevState = GetConnectionState();
	{
		FScopeLock SendLock(&SendCriticalSection);
		ConnectionState = InNewState;
	}

	if (PrevState != ConnectionState)
	{
		ConnectionStateChangedDelegate.ExecuteIfBound();
	}
}

FString FStormSyncTransportClientSocket::GetSocketReadableErrorCode()
{
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	const ESocketErrors LastError = SocketSubsystem->GetLastErrorCode();
	return FString::Printf(TEXT("%s (%d)"), SocketSubsystem->GetSocketError(LastError), static_cast<int32>(LastError));
}

FString FStormSyncTransportClientSocket::GetSocketReadableConnectionState(const ESocketConnectionState InState)
{
	switch (InState)
	{
		case SCS_NotConnected: return TEXT("SCS_NotConnected");
		case SCS_Connected: return TEXT("SCS_Connected");
		case SCS_ConnectionError: return TEXT("SCS_ConnectionError (Indicates that the end point refused the connection or couldn't be reached)");
		default: return TEXT("Unknown State");
	}
}
