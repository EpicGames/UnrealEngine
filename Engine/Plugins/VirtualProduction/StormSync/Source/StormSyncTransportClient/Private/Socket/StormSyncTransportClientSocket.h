// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "StormSyncCommonTypes.h"
#include "StormSyncTransportSettings.h"

/**
 * Implements a TCP message tunnel connection.
 *
 * Mainly used to send a raw buffer serialized from StormSyncCoreUtils methods to transfer a content spak on a remote instance.
 */
class FStormSyncTransportClientSocket : public FRunnable, public TSharedFromThis<FStormSyncTransportClientSocket>
{
public:
	/** Event delegate that is executed when socket is closed */
	DECLARE_DELEGATE_OneParam(FStormSyncOnConnectionClosed, const FIPv4Endpoint&)

	/** Delegate type for announcing a connection state change */
	DECLARE_DELEGATE(FStormSyncOnConnectionStateChanged)
	
	/** Delegate type triggered when the underlying socket receives data back from remote, representing the raw size received so far */
	DECLARE_DELEGATE_OneParam(FStormSyncOnReceivedSize, const int32)
	
	/** Delegate type triggered when the underlying socket receives transfer_complete packet from remote, indicating transfer is done */
	DECLARE_DELEGATE(FStormSyncOnTransferComplete)

	/** Connection state */
	enum EConnectionState
	{
		// Initial state before establishing any connection to remote
		State_Connecting,
		
		// Connected and ready to send buffers over the connection
		State_Connected,

		// Disconnected with reconnect pending, state after a reconnect w/o having received anything new yet
		State_DisconnectReconnectPending,

		// Disconnected
		State_Closed
	};

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InEndpoint The IP endpoint of the remote listener
	 */
	explicit FStormSyncTransportClientSocket(const FIPv4Endpoint& InEndpoint);
	
	/** Virtual destructor. */
	virtual ~FStormSyncTransportClientSocket();

	/** Starts processing of the connection. Needs to be called immediately after construction */
	void StartTransport();

	/** Stops processing of the connection */
	void StopTransport();

	/** Returns whether processing of the connection is currently active */
	bool IsTransportRunning() const;

	//~ Begin FRunnable interface
	virtual bool Init() override;
	virtual void Stop() override;
	virtual uint32 Run() override;
	//~ End FRunnable interface

	/**
	 * Main tick handler and throttled by Run(). Listens for incoming data on our connected socket.
	 *
	 * Returns false if RequestEngineExit was called.
	 */
	bool Tick(FString& StopReason);

	/** Attempts to reconnect to configured endpoint */
	bool TryReconnect();

	/**
	 * Create the tcp socket and tries to establish connection on the configured endpoint.
	 *
	 * @return true if we wera able to connect
	 */
	bool Connect();

	/** Explicitly close the underlying socket (usually prior to reconnecting) */
	void CloseSocket(const bool bShouldTriggerEvent = false);

	/** Returns whether socket is valid and connection status is ok */
	bool IsConnected() const;

	/**
	 * Main utility method to send a buffer over the tcp connection (Connection must have been established before using Connect())
	 *
	 * @param InBuffer The raw buffer to send
	 */
	void SendBuffer(const FStormSyncBuffer& InBuffer);

	/**
	 * Gets the current state of the connection.
	 *
	 * @return EConnectionState current state.
	 */
	EConnectionState GetConnectionState() const;
	
	/**
	 * Returns a human readable string from a connection state
	 *
	 * @param State the state to convert to string
	 */
	static FString GetReadableConnectionState(EConnectionState State);

	/**
	 * Gets the delegate which will be called whenever the underlying socket is closed
	 *
	 * @return FOnConnectionClosedEvent delegate
	 */
	FStormSyncOnConnectionClosed& OnConnectionClosed() { return ConnectionClosedDelegate; }

	/**
	 * Gets the delegate which will be called whenever the connection state changes
	 *
	 * @return FStormSyncOnConnectionStateChanged delegate
	 */
	FStormSyncOnConnectionStateChanged& OnConnectionStateChanged() { return ConnectionStateChangedDelegate; }
	
	/**
	 * Gets the delegate which will be called whenever we get back size received by the remote endpoint
	 *
	 * @return FStormSyncOnConnectionStateChanged delegate
	 */
	FStormSyncOnReceivedSize& OnReceivedSizeDelegate() { return ReceivedSizeDelegate; }
	
	/**
	 * Gets the delegate which will be called whenever remote fully received the buffer, indicating transfer is complete
	 *
	 * @return FStormSyncOnConnectionStateChanged delegate
	 */	
	FStormSyncOnTransferComplete& OnTransferComplete() { return TransferCompleteDelegate; }

	/**
	 * Gets the IP address and port of the remote connection
	 *
	 * @return FIPv4Endpoint remote connection's IP address and port
	 */
	FIPv4Endpoint GetRemoteEndpoint() const { return RemoteEndpoint; }

	/**
	 * Gets the total number of bytes received from this connection.
	 *
	 * @return Number of bytes.
	 */
	uint64 GetTotalBytesReceived() const { return TotalBytesReceived; }

	/**
	 * Gets the total number of bytes sent on this connection.
	 *
	 * @return Number of bytes.
	 */
	uint64 GetTotalBytesSent() const { return TotalBytesReceived; }

	/**
	 * Are we currently sending data on this connection ?
	 *
	 * Sending new data should only be allowed if we're not busy sending already on this connection.
	 */
	bool IsSending() const	{ return bSending; }

private:
	/** The socket description when used with FTcpSocketBuilder */
	static constexpr const TCHAR* SocketDescription = TEXT("stormsync-tcp-client");
	
	// Note: For these few following variables, it may make sense to move them as part of the developer settings, in an Advanced Category.
	// ~ https://github.com/Geodesic-Games/StormSync/pull/23/files#r905356914
	
	/** Default to roughly 4 Mb. Socket Send / Receive Buffer Size */
	const int32 SocketBufferSize = 4 * 1024 * 1024;

	/** Maximum amount of time to wait when we block waiting for read on the underlying socket */
	const double SocketWaitForReadTimeSeconds = 1.0;
	
	/** Maximum amount of time to wait when we block waiting for write on the underlying socket */
	const double SocketWaitForWriteTimeSeconds = 2.0;
	
	/** Current connection state */
	EConnectionState ConnectionState;
	
	/** Holds the IP address (and port) that the socket will be bound to. */
	FIPv4Endpoint RemoteEndpoint;

	/** Holds the thread object. */
	FRunnableThread* Thread;

	/** Holds the connection socket. */
	FSocket* Socket;

	/** For the thread */
	std::atomic<bool> bStopping = true;

	/** Represents the total number of bytes for a buffer that is sent to a remote */
	int32 TotalBytes = 0;

	/** Represents the number of bytes received for a buffer on a remote */
	int32 CurrentBytes = 0;

	/** Holds the total number of bytes received from the connection. */
	uint64 TotalBytesReceived = 0;

	/** Holds the total number of bytes sent to the connection. */
	uint64 TotalBytesSent = 0;

	/** Delay before re-establishing connection if it drops, 0 disables */
	uint32 ConnectionRetryDelay;

	/** Delegate trigger when underlying socket is closed */
	FStormSyncOnConnectionClosed ConnectionClosedDelegate;
	
	/** Connection state changed delegate */
	FStormSyncOnConnectionStateChanged ConnectionStateChangedDelegate;
	
	/** Delegate triggered when receiving back from server the total amount of bytes received so far */
	FStormSyncOnReceivedSize ReceivedSizeDelegate;
	
	/** Delegate triggered when server responds back with transfer complete packet */
	FStormSyncOnTransferComplete TransferCompleteDelegate;

	/** Cached version of plugin settings */
	const UStormSyncTransportSettings* Settings;

	/** Are we currently sending data on this connection */
	bool bSending = false;

	/**
	 * Main utility method to send an arbitrary data buffer over the tcp socket.
	 *
	 * If bPrependSize is set to true (the default), the raw size of the buffer is sent as an "header" in the tcp stream, so that remote
	 * knows how many bytes it's gonna receive.
	 *
	 * Connection must have been established before using Connect().
	 */
	bool Send(const uint8* Data, uint32 Size, bool bPrependSize = true);

	/** Try to send data, but if all data is not sent in one go, block on send until data is sent or an error occurs */
	bool BlockingSend(const uint8* Data, int32 BytesToSend);

	/**
	 * Called from Tick and handles an incoming chunk of data.
	 *
	 * We expect server to respond with the overall number of bytes it received so far, which we use
	 * here to update the notification widget progress bar.
	 */
	void ParseIncomingBytes(const int32 InNumBytesRead, const TArray<uint8>& InBytes);

	/**
	 * Parses an incoming message from server (JSON serialized struct). It is right now:
	 *
	 * - (state) used on connection, while we wait for server to respond with a FStormSyncTransportTcpStatePacket indicating connection was successful.
	 * - (size) used when remote is receiving bytes, responding back with the total amount of bytes received so far.
	 *
	 * On "size" packets, the ReceivedSizeDelegate is triggered with the amount of bytes received.
	 */
	bool ParseIncomingMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint);

	/** Updates connection state and triggers delegate if it changed */
	void UpdateConnectionState(EConnectionState InNewState);

	/** Internal helper returning a formatted string with last error encountered by underlying socket subsystem (Format: ErrorCodeString (ErrorCode)) */
	static FString GetSocketReadableErrorCode();
	
	/** Internal helper returning a formatted string with current socket connection state (Format: State (Description)) */
	static FString GetSocketReadableConnectionState(ESocketConnectionState InState);

	/** Critical section preventing multiple threads from sending simultaneously */
	FCriticalSection SendCriticalSection;
};
