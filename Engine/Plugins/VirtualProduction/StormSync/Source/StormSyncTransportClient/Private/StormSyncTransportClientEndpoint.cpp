// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportClientEndpoint.h"

#include "Algo/Transform.h"
#include "Async/Async.h"
#include "IMessagingModule.h"
#include "MessageEndpointBuilder.h"
#include "Misc/PackageName.h"
#include "Serialization/MemoryWriter.h"
#include "Services/StormSyncPullMessageService.h"
#include "Services/StormSyncPushMessageService.h"
#include "Services/StormSyncStatusMessageService.h"
#include "Socket/StormSyncTransportClientSocket.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncTransportClientLog.h"
#include "StormSyncTransportMessages.h"

#define LOCTEXT_NAMESPACE "StormSyncTransportClientEndpoint"

FStormSyncTransportClientEndpoint::FStormSyncTransportClientEndpoint()
{
	MessageBusPtr = IMessagingModule::Get().GetDefaultBus();
	check(MessageBusPtr.IsValid());
}

FStormSyncTransportClientEndpoint::~FStormSyncTransportClientEndpoint()
{
	ShutdownMessaging();
}

void FStormSyncTransportClientEndpoint::InitializeMessaging(const FString& InEndpointFriendlyName)
{
	// Message endpoint already created
	if (MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Warning, "FStormSyncTransportClientEndpoint::InitializeMessaging - Message endpoint already initialized");
		return;
	}

	const IMessageBusPtr MessageBus = MessageBusPtr.Pin();
	if (!MessageBus.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::InitializeMessaging - Default message bus is invalid");
		return;
	}

	// Create endpoint builder
	const FString MessageEndpointName = FString::Printf(TEXT("StormSync%sEndpoint"), *InEndpointFriendlyName);
	UE_LOGF(LogStormSyncClient, Verbose, "FStormSyncTransportClientEndpoint::InitializeMessaging - Setting up %ls message endpoint", *MessageEndpointName);

	FMessageEndpointBuilder EndpointBuilder = FMessageEndpoint::Builder(*MessageEndpointName, MessageBus.ToSharedRef())
		.ReceivingOnThread(ENamedThreads::Type::GameThread);

	EndpointBuilder.Handling<FStormSyncTransportPongMessage>(this, &FStormSyncTransportClientEndpoint::HandlePongMessage);

	// Message service handlers
	StatusService = MakeShared<FStormSyncStatusMessageService>(AsShared());
	StatusService->InitializeMessageEndpoint(EndpointBuilder);

	PushService = MakeShared<FStormSyncPushMessageService>(AsShared());
	PushService->InitializeMessageEndpoint(EndpointBuilder);
	
	PullService = MakeShared<FStormSyncPullMessageService>(AsShared());
	PullService->InitializeMessageEndpoint(EndpointBuilder);

	// Build the endpoint
	MessageEndpoint = EndpointBuilder.Build();
	check(MessageEndpoint.IsValid());

	// Message subscribes
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::InitializeMessaging - Subscribe to messages");
	MessageEndpoint->Subscribe<FStormSyncTransportStatusPing>();
}

void FStormSyncTransportClientEndpoint::ShutdownMessaging()
{
	// Disable the RemoteEndpoint message handling since the message could keep it alive a bit.
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Disable();
		MessageEndpoint.Reset();
	}

	if (StatusService.IsValid())
	{
		StatusService.Reset();
	}

	// Cleaning up of the connections map
	Connections.Empty();
}

bool FStormSyncTransportClientEndpoint::IsRunning() const
{
	return MessageEndpoint.IsValid() && MessageEndpoint->IsEnabled();
}

TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> FStormSyncTransportClientEndpoint::GetMessageEndpoint() const
{
	return MessageEndpoint;
}

void FStormSyncTransportClientEndpoint::RequestStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& InDoneDelegate) const
{
	if (!StatusService.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::RequestStatus - Status message service is invalid");
		return;
	}

	StatusService->RequestStatus(InRemoteAddress, InPackageNames, InDoneDelegate);
}

void FStormSyncTransportClientEndpoint::RequestPushPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPushComplete& InDoneDelegate) const
{
	if (!PushService.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::RequestPushPackages - Push message service is invalid");
		return;
	}

	PushService->RequestPushPackages(InRemoteAddress, InPackageDescriptor, InPackageNames, InDoneDelegate);
}

void FStormSyncTransportClientEndpoint::RequestPullPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPullComplete& InDoneDelegate) const
{
	if (!PullService.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::RequestPullPackages - Pull message service is invalid");
		return;
	}

	PullService->RequestPullPackages(InRemoteAddress, InPackageDescriptor, InPackageNames, InDoneDelegate);
}

void FStormSyncTransportClientEndpoint::AbortStatusRequest(const FGuid& InStatusRequestId) const
{
	if (!StatusService.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::AbortStatusRequest - Status message service is invalid");
		return;
	}

	StatusService->AbortStatusRequest(InStatusRequestId);
}

void FStormSyncTransportClientEndpoint::AbortPushRequest(const FGuid& InPushRequestId) const
{
	if (!PushService.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::AbortPushRequest - Status message service is invalid");
		return;
	}

	PushService->AbortPushRequest(InPushRequestId);
}

void FStormSyncTransportClientEndpoint::AbortPullRequest(const FGuid& InPullRequestId) const
{
	if (!PullService.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::AbortPullRequest - Status message service is invalid");
		return;
	}

	PullService->AbortPullRequest(InPullRequestId);
}

void FStormSyncTransportClientEndpoint::StartSendingBuffer(const FStormSyncTransportSyncResponse& InMessage, TSharedPtr<FStormSyncTransportClientSocket>& OutActiveConnection, const FOnStormSyncSendBufferCallback& InDoneDelegate)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::StartSendingBuffer - Message: %ls", *InMessage.ToString());
	
	if (InMessage.Modifiers.IsEmpty())
	{
		// Remote sent back empty list of modifiers, meaning pak are synced. Do nothing.
		// Technically not an error, but caller should have checked this previously
		InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncSendingBufferPayload>(LOCTEXT("Error_Modifiers_Empty", "Remote sent back empty list of modifiers, meaning pak are synced. Do nothing.")));
		return;
	}

	const TSharedPtr<FStormSyncTransportClientSocket> ActiveConnection = GetActiveConnection(InMessage.HostAddress, InMessage.HostAdapterAddresses);
	if (!ActiveConnection)
	{
		// We were not able to create the connection properly (most likely because of a socket connection issue)
		// Don't proceed further
		const FText ErrorText = FText::Format(LOCTEXT("Error_Active_Connection", "Error trying to determine an active connection for {0}"), FText::FromString(InMessage.ToString()));
		InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncSendingBufferPayload>(ErrorText));
		return;
	}

	// Now that we have an active connection, pass it down to caller (even if we can still fail down the road)
	OutActiveConnection = ActiveConnection;
	
	FString ServerAddress = ActiveConnection->GetRemoteEndpoint().ToString();
	// Only allowed to send if we are connected and not currently sending on this connection
	if (ActiveConnection->IsSending())
	{
		// Currently sending data on this remote, don't proceed further
		InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncSendingBufferPayload>(LOCTEXT("Error_Already_Sending", "We are currently sending data to %s. Please wait for it to complete before trying to send new data again.")));
		return;
	}

	// Convert modifiers into just their package names
	TArray<FName> PackageNames = GetPackageNamesFromModifierInfos(InMessage.Modifiers);

	// Filter out any invalid packages (eg. not existing on disk). In case of pull, remote may be sending modifier addition
	// and we want to prevent pak buffer creation to fail on these
	PackageNames = PackageNames.FilterByPredicate([](const FName& PackageName)
	{
		return FPackageName::DoesPackageExist(PackageName.ToString());
	});

	// Ensure delegates run on game thread
	FString RemoteHostName = InMessage.HostName;
	AsyncTask(ENamedThreads::GameThread, [ServerAddress, HostName = MoveTemp(RemoteHostName), NumPackageNames = PackageNames.Num()]()
	{
		FStormSyncCoreDelegates::OnPreStartSendingBuffer.Broadcast(ServerAddress, HostName, NumPackageNames);
	});

	TWeakPtr<FStormSyncTransportClientEndpoint, ESPMode::ThreadSafe> LocalWeakThis(SharedThis(this));
	Async(EAsyncExecution::Thread, [LocalWeakThis, ServerAddress, PackageNames, InDoneDelegate]()
	{
		FText ErrorText;
		FStormSyncBuffer PakBuffer;
		FMemoryWriter64 PakArchiveWriter(PakBuffer);
		if (!FStormSyncCoreUtils::CreatePakBuffer(PackageNames, PakArchiveWriter, ErrorText))
		{
			UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::StartSendingBuffer - CreatePakBuffer Error: %ls", *ErrorText.ToString());
			InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncSendingBufferPayload>(ErrorText));
			return;
		}
		
		const TSharedPtr<FStormSyncTransportClientEndpoint, ESPMode::ThreadSafe> StrongThis = LocalWeakThis.Pin();
		if (!StrongThis.IsValid())
		{
			InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncSendingBufferPayload>(LOCTEXT("StartSendingBuffer_Error_WeakThis", "Internal error trying to get a hold on local weak this.")));
			return;
		}
		
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::StartSendingBuffer - Try sending on %ls", *ServerAddress);
		StrongThis->SendTcpBuffer(ServerAddress, PakBuffer);
		
		InDoneDelegate.ExecuteIfBound(MakeShared<FStormSyncSendingBufferPayload>());
	});
}

TSharedPtr<FStormSyncTransportClientSocket> FStormSyncTransportClientEndpoint::GetOrCreateClientSocket(const FString& InAddress)
{
	FIPv4Endpoint Endpoint(FIPv4Address::Any, 0);
	if (!FIPv4Endpoint::Parse(InAddress, Endpoint))
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientSocket::GetOrCreateClientSocket - Failed to parse endpoint '%ls'.", *InAddress);
		return nullptr;
	}

	FScopeLock ScopeLock(&ConnectionsCriticalSection);

	// Already active
	if (Connections.Contains(Endpoint))
	{
		return Connections.FindChecked(Endpoint);
	}

	const TSharedPtr<FStormSyncTransportClientSocket> Connection = MakeShared<FStormSyncTransportClientSocket>(Endpoint);
	check(Connection);

	Connection->OnConnectionClosed().BindThreadSafeSP(this, &FStormSyncTransportClientEndpoint::OnConnectionClosedForSocket);
	Connection->OnConnectionStateChanged().BindThreadSafeSP(this, &FStormSyncTransportClientEndpoint::OnConnectionStateChanged, Connection);
	Connection->OnReceivedSizeDelegate().BindThreadSafeSP(this, &FStormSyncTransportClientEndpoint::OnConnectionReceivedBytes, InAddress);
	Connection->OnTransferComplete().BindThreadSafeSP(this, &FStormSyncTransportClientEndpoint::OnTransferComplete, InAddress);

	Connection->StartTransport();

	if (!Connection->Connect())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientSocket::GetOrCreateClientSocket - Connect failed on socket for %ls", *Endpoint.ToString());
		return nullptr;
	}

	Connections.Add(Endpoint, Connection);
	return Connection;
}

TSharedPtr<FStormSyncTransportClientSocket> FStormSyncTransportClientEndpoint::GetActiveConnection(const FString& InHostAddress, const TArray<FString>& InHostAdapterAddresses)
{
	TArray<FString> ServerAddresses;

	// Check the host server address it is listening on, and determine the remote endpoint which we should try to connect to.
	if (InHostAddress.StartsWith(FIPv4Address::Any.ToString()))
	{
		// If the address server is listening on is the 0.0.0.0 default route,
		// use the list of local addresses associated with the adapters on the remote computer
		ServerAddresses.Append(InHostAdapterAddresses);
	}
	else
	{
		// Otherwise, just use the configured host address
		ServerAddresses.Add(InHostAddress);
	}

	// Try establish connection on each, and stop on the first successful one
	for (FString ServerAddress : ServerAddresses)
	{
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::GetActiveConnection - Try connection on %ls", *ServerAddress);
		if (TSharedPtr<FStormSyncTransportClientSocket> Connection = GetOrCreateClientSocket(ServerAddress))
		{
			UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::GetActiveConnection - Succesfful connection on %ls", *ServerAddress);
			return Connection;
		}
	}

	return nullptr;
}

void FStormSyncTransportClientEndpoint::HandlePongMessage(const FStormSyncTransportPongMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::HandlePongMessage - Received pong Message %ls from %ls", *InMessage.ToString(), *InMessageContext->GetSender().ToString());
}

TArray<FName> FStormSyncTransportClientEndpoint::GetPackageNamesFromModifierInfos(const TArray<FStormSyncFileModifierInfo>& InModifierInfos)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::GetPackageNamesFromModifierInfos - ModifierInfos: %d", InModifierInfos.Num());

	TArray<FName> PackageNames;
	Algo::Transform(InModifierInfos, PackageNames, [](const FStormSyncFileModifierInfo& ModifierInfo)
	{
		return ModifierInfo.FileDependency.PackageName;
	});

	return PackageNames;
}

void FStormSyncTransportClientEndpoint::SendTcpBuffer(const FString& InAddress, const FStormSyncBuffer& InBuffer)
{
	const TSharedPtr<FStormSyncTransportClientSocket> Connection = GetOrCreateClientSocket(InAddress);
	if (!Connection)
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientEndpoint::SendTcpBuffer - Failed to GetOrCreateClientSocket client socket to '%ls'. TCP Client sending buffer will be disabled!", *InAddress);
		return;
	}

	const FIPv4Endpoint Endpoint = Connection->GetRemoteEndpoint();
	const FStormSyncTransportClientSocket::EConnectionState State = Connection->GetConnectionState();

	UE_LOGF(LogStormSyncClient, 
		Display,
		"FStormSyncTransportClientEndpoint::SendTcpBuffer for %ls (State: %ls, Sending: %ls)",
		*Endpoint.ToString(),
		*Connection->GetReadableConnectionState(State),
		Connection->IsSending() ? TEXT("true") : TEXT("false")
	);

	// Only allowed to send if we are connected and not currently sending on this connection
	if (!Connection->IsSending())
	{
		// Ensure delegates run on game thread
		const int32 BufferSize = InBuffer.Num();
		FString LocalAddress = InAddress;
		AsyncTask(ENamedThreads::GameThread, [Address = MoveTemp(LocalAddress), BufferSize]()
		{
			FStormSyncCoreDelegates::OnStartSendingBuffer.Broadcast(Address, BufferSize);
		});

		// Start sending now
		Connection->SendBuffer(InBuffer);
	}
	else
	{
		UE_LOGF(LogStormSyncClient, 
			Error,
			"FStormSyncTransportClientEndpoint::SendTcpBuffer - We are currently sending data to %ls. Please wait for it to complete before trying to send new data again.",
			*Endpoint.ToString()
		);
	}
}

void FStormSyncTransportClientEndpoint::OnConnectionClosedForSocket(const FIPv4Endpoint& InEndpoint)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::OnConnectionClosedForSocket - TCP socket closed for %ls", *InEndpoint.ToString());
}

void FStormSyncTransportClientEndpoint::OnConnectionStateChanged(const TSharedPtr<FStormSyncTransportClientSocket> InConnection)
{
	const FIPv4Endpoint Endpoint = InConnection->GetRemoteEndpoint();
	const FStormSyncTransportClientSocket::EConnectionState State = InConnection->GetConnectionState();

	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::OnConnectionStateChanged for %ls (State: %ls)", *Endpoint.ToString(), *InConnection->GetReadableConnectionState(State));

	const bool bHasConnectionRetryDelay = GetDefault<UStormSyncTransportSettings>()->HasConnectionRetryDelay();

	// Clean up local pool of connected clients when they are closed, only if we don't have a retry delay
	if (!bHasConnectionRetryDelay && State == FStormSyncTransportClientSocket::State_Closed)
	{
		FScopeLock ScopeLock(&ConnectionsCriticalSection);
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::OnConnectionStateChanged - Closed, dispose of %ls connection", *InConnection->GetRemoteEndpoint().ToString());
		Connections.Remove(InConnection->GetRemoteEndpoint());
	}
}

void FStormSyncTransportClientEndpoint::OnConnectionReceivedBytes(const int32 InReceivedSize, const FString InConnectionAddress)
{
	FStormSyncCoreDelegates::OnReceivingBytes.Broadcast(InConnectionAddress, InReceivedSize);
}

void FStormSyncTransportClientEndpoint::OnTransferComplete(const FString InConnectionAddress)
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientEndpoint::OnTransferComplete - TCP socket done transfering for %ls", *InConnectionAddress);
	FStormSyncCoreDelegates::OnTransferComplete.Broadcast(InConnectionAddress);
}

#undef LOCTEXT_NAMESPACE