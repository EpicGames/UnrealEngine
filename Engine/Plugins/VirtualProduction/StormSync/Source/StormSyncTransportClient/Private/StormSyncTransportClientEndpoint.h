// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportClientLocalEndpoint.h"
#include "StormSyncTransportMessages.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

class FStormSyncPullMessageService;
class FStormSyncPushMessageService;
class FStormSyncStatusMessageService;
class FStormSyncTransportClientSocket;
class IMessageBus;
class IMessageContext;
struct FMessageAddress;

using IMessageBusPtr = TSharedPtr<IMessageBus, ESPMode::ThreadSafe>;

/** Struct payload for StartSendingBuffer() async operation */
struct FStormSyncSendingBufferPayload
{
	bool bSuccess = true;
	FText ErrorText;
	
	FStormSyncSendingBufferPayload() = default;
	~FStormSyncSendingBufferPayload() {}


	explicit FStormSyncSendingBufferPayload(const FText& ErrorText)
		: bSuccess(false)
		, ErrorText(ErrorText)
	{
	}
};

DECLARE_DELEGATE_OneParam(FOnStormSyncSendBufferCallback, const TSharedPtr<FStormSyncSendingBufferPayload>&)

/**
 * Implements a local endpoint for Storm Sync Client.
 */
class FStormSyncTransportClientEndpoint : public IStormSyncTransportClientLocalEndpoint
{
public:
	FStormSyncTransportClientEndpoint();
	virtual ~FStormSyncTransportClientEndpoint() override;

	/** Initializes the message endpoint. */
	void InitializeMessaging(const FString& InEndpointFriendlyName = TEXT("Client"));

	/** Initializes the message endpoint. */
	void ShutdownMessaging();

	//~ Begin IStormSyncTransportLocalEndpoint
	/** Returns whether message endpoint is currently active */
	virtual bool IsRunning() const override;

	/** Returns underlying message endpoint */
	virtual TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const override;
	//~ End IStormSyncTransportLocalEndpoint
	
	//~ Begin IStormSyncTransportClientLocalEndpoint
	virtual void RequestStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& InDoneDelegate) const override;
	virtual void RequestPushPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPushComplete& InDoneDelegate) const override;
	virtual void RequestPullPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPullComplete& InDoneDelegate) const override;
	virtual void AbortStatusRequest(const FGuid& InStatusRequestId) const override;
	virtual void AbortPushRequest(const FGuid& InPushRequestId) const override;
	virtual void AbortPullRequest(const FGuid& InPullRequestId) const override;
	//~ End IStormSyncTransportClientLocalEndpoint
	
	/**
	 * Async operation to initialize a new tcp connection based on the provided sync response (either a push or pull case)
	 *
	 * Will attempt to:
	 *
	 * - Find the best suited remote tcp address to connect to based on provided message HostAddress and Adapters
	 * - Creates a new buffer based on the provided message sync modifiers
	 * - Starts sending buffer via tcp to the remote storm sync tcp server address
	 *
	 * The InDoneDelegate callback can be used to gather any errors that can arise during the process. It will be invoked with a payload (FStormSyncSendingBufferPayload) describing the error in case of errors,
	 * or simply with an empty payload marked as "success" in case of no errors.
	 *
	 * @param InMessage The sync response message to initialize the tcp connection and create the pak buffer from (should include info such remote network host / adapters, a list of sync modifiers, etc.)
	 * @param OutActiveConnection Will be filled with a shared ptr to the underlying active socket connection (FStormSyncTransportClientSocket)
	 * @param InDoneDelegate An optional callback that can be used to be notified about completion (with error or success payload)
	 * 
	 */
	void StartSendingBuffer(const FStormSyncTransportSyncResponse& InMessage, TSharedPtr<FStormSyncTransportClientSocket>& OutActiveConnection, const FOnStormSyncSendBufferCallback& InDoneDelegate = FOnStormSyncSendBufferCallback());
	
private:
	/** Internal payload struct to gather CreatePakBuffer() result, that is usually run in a background thread */
	struct FAsyncBufferPayload
	{
		bool bSuccess = true;
		FText ErrorText;
		TArray<uint8> PakBuffer;

		FAsyncBufferPayload() = default;
	};
	
	/** Map of currently connected and active clients used to send a raw buffer over tcp */
	TMap<FIPv4Endpoint, TSharedPtr<FStormSyncTransportClientSocket>> Connections;
	
	/**
	 * Holds a weak pointer to the message bus.
	 *
	 * @note Since we're using the default message bus, this might be not necessary
	 */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr;

	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Status message service handler */
	TSharedPtr<FStormSyncStatusMessageService, ESPMode::ThreadSafe> StatusService;
	
	/** Push message service handler */
	TSharedPtr<FStormSyncPushMessageService, ESPMode::ThreadSafe> PushService;
	
	/** Pull message service handler */
	TSharedPtr<FStormSyncPullMessageService, ESPMode::ThreadSafe> PullService;

	/** Helper to return a currently active connection or create a new one */
	TSharedPtr<FStormSyncTransportClientSocket> GetOrCreateClientSocket(const FString& InAddress);

	/** Goes over connected clients and returns the first successful connection based on host address and possible adapters config from remote */
	TSharedPtr<FStormSyncTransportClientSocket> GetActiveConnection(const FString& InHostAddress, const TArray<FString>& InHostAdapterAddresses);

	/** Transforms a list of modifier infos into their Package Name equivalent */
	static TArray<FName> GetPackageNamesFromModifierInfos(const TArray<FStormSyncFileModifierInfo>& InModifierInfos);

	/** Callback handler to receive FStormSyncTransportPongMessage messages */
	void HandlePongMessage(const FStormSyncTransportPongMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);
	
	/** Forwards sending of raw buffer over a tcp connection */
	void SendTcpBuffer(const FString& InAddress, const FStormSyncBuffer& InBuffer);

	/** Handler for when a tcp socket is closing */
	void OnConnectionClosedForSocket(const FIPv4Endpoint& InEndpoint);
	
	/** Handler for when a connection state changes */
	void OnConnectionStateChanged(TSharedPtr<FStormSyncTransportClientSocket> InConnection);

	/** Handler for a connection receiving bytes */
	void OnConnectionReceivedBytes(const int32 InReceivedSize, const FString InConnectionAddress);
	
	/** Handler for a connection done with tcp transfer */
	void OnTransferComplete(const FString InConnectionAddress);
	
	/** Critical section preventing multiple threads from adding / removing simultaneously */
	FCriticalSection ConnectionsCriticalSection;
};
