// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportClientLog.h"

DEFINE_LOG_CATEGORY(LogStormSyncClient);
