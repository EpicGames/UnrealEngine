// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportClientModule.h"

#include "StormSyncCommandLineUtils.h"
#include "StormSyncCoreUtils.h"
#include "IStormSyncTransportCoreModule.h"
#include "IStormSyncTransportLocalEndpoint.h"
#include "MessageEndpoint.h"
#include "Misc/CoreDelegates.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncTransportClientEndpoint.h"
#include "StormSyncTransportClientLog.h"
#include "StormSyncTransportMessages.h"

#define LOCTEXT_NAMESPACE "StormSyncTransportClientModule"

void FStormSyncTransportClientModule::StartupModule()
{
	// Auto-start the client unless in a commandlet.
	if (!IsRunningCommandlet())
	{
		StartClientEndpoint(DefaultClientEndpointName);
	}

	RegisterConsoleCommands();
	
	// Register for engine initialization completed so we can broadcast presence over the network from this client to other clients
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStormSyncTransportClientModule::OnPostEngineInit);

	FStormSyncCoreDelegates::OnServiceDiscoveryReceivedWakeup.AddRaw(this, &FStormSyncTransportClientModule::OnWakeup);
	
	IStormSyncTransportCoreModule::Get().OnGetClientEndpointMessageAddress().BindRaw(this, &FStormSyncTransportClientModule::GetClientEndpointMessageAddressId);
}

void FStormSyncTransportClientModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FStormSyncCoreDelegates::OnServiceDiscoveryReceivedWakeup.RemoveAll(this);

	if (IStormSyncTransportCoreModule::IsAvailable())
	{
		IStormSyncTransportCoreModule::Get().OnGetClientEndpointMessageAddress().Unbind();
	}
	
	if (ClientEndpoint.IsValid())
	{
		ClientEndpoint.Reset();
	}

	UnregisterConsoleCommands();
}

void FStormSyncTransportClientModule::StartClientEndpoint(const FString& InEndpointFriendlyName)
{
	if (!ClientEndpoint.IsValid())
	{
		ClientEndpoint = CreateClientLocalEndpoint(InEndpointFriendlyName);
		if (!ClientEndpoint.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::StartClientEndpoint - Failed to create Client Local Endpoint");
			return;
		}
	}
	
	if (bEngineInitComplete)
	{
		PublishStatusPingMessage();
	}
}

TSharedPtr<IStormSyncTransportClientLocalEndpoint> FStormSyncTransportClientModule::CreateClientLocalEndpoint(const FString& InEndpointFriendlyName) const
{
	TSharedPtr<FStormSyncTransportClientEndpoint, ESPMode::ThreadSafe> Endpoint = MakeShared<FStormSyncTransportClientEndpoint>();
	check(Endpoint.IsValid());
	Endpoint->InitializeMessaging(InEndpointFriendlyName);
	return Endpoint;
}

FString FStormSyncTransportClientModule::GetClientEndpointMessageAddressId() const
{
	return ClientEndpoint.IsValid() && ClientEndpoint->IsRunning() ? ClientEndpoint->GetMessageEndpoint()->GetAddress().ToString() : TEXT("");
}

FMessageEndpointSharedPtr FStormSyncTransportClientModule::GetClientMessageEndpoint() const
{
	if (!ClientEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Warning, "FStormSyncTransportClientModule::GetClientMessageEndpoint - Client endpoint not valid");
		return nullptr;
	}

	if (!ClientEndpoint->IsRunning())
	{
		UE_LOGF(LogStormSyncClient, Warning, "FStormSyncTransportClientModule::GetClientMessageEndpoint - Client endpoint not running");
		return nullptr;
	}

	return ClientEndpoint->GetMessageEndpoint();
}

void FStormSyncTransportClientModule::SynchronizePackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames) const
{
	const FMessageEndpointSharedPtr MessageEndpoint = GetClientMessageEndpoint();

	TArray<FName> LocalPackageNames = InPackageNames;
	FStormSyncPackageDescriptor LocalPackageDescriptor = InPackageDescriptor;

	FStormSyncCoreUtils::GetAvaFileDependenciesAsync(InPackageNames)
	.Then([MessageEndpoint, PackageNames = MoveTemp(LocalPackageNames), PackageDescriptor = MoveTemp(LocalPackageDescriptor)](const TFuture<TArray<FStormSyncFileDependency>> Result)
	{
		const TArray<FStormSyncFileDependency> FileDependencies = Result.Get();
		if (FileDependencies.IsEmpty())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::SynchronizePackages - Async FileDependencies is empty, something went wrong");
			return;
		}

		if (!MessageEndpoint.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::SynchronizePackages - Unable to get client message endpoint");
			return;
		}

		// Build sync request message that is going to be sent over the network for a specific recipient
		TUniquePtr<FStormSyncTransportSyncRequest> SyncRequestMessage(FMessageEndpoint::MakeMessage<FStormSyncTransportSyncRequest>(PackageNames, PackageDescriptor));
		if (!SyncRequestMessage.IsValid())
		{
			UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::SynchronizePackages - Push request message is invalid");
			return;
		}

		SyncRequestMessage->PackageDescriptor.Dependencies = FileDependencies;

		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::SynchronizePackages - FileDependencies: %d", FileDependencies.Num());
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::SynchronizePackages - Message: %ls", *SyncRequestMessage->ToString());
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::SynchronizePackages - Syncing package descriptor %ls", *SyncRequestMessage->PackageDescriptor.ToString());
		MessageEndpoint->Publish(SyncRequestMessage.Release());
	});
}

void FStormSyncTransportClientModule::PushPackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FMessageAddress& InMessageAddress, const FOnStormSyncPushComplete& InDoneDelegate) const
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::PushPackages - PackageDescriptor: %ls, InPackageNames: %d, MessageAddressId: %ls", *InPackageDescriptor.ToString(), InPackageNames.Num(), *InMessageAddress.ToString());

	if (!ClientEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::PushPackages - Unable to get client endpoint");
		return;
	}

	ClientEndpoint->RequestPushPackages(InMessageAddress, InPackageDescriptor, InPackageNames, InDoneDelegate);
}

void FStormSyncTransportClientModule::PullPackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FMessageAddress& InMessageAddress, const FOnStormSyncPullComplete& InDoneDelegate) const
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::PullPackages - PackageDescriptor: %ls, InPackageNames: %d, MessageAddressId: %ls", *InPackageDescriptor.ToString(), InPackageNames.Num(), *InMessageAddress.ToString());

	if (!ClientEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::PullPackages - Unable to get client endpoint");
		return;
	}

	ClientEndpoint->RequestPullPackages(InMessageAddress, InPackageDescriptor, InPackageNames, InDoneDelegate);
}

void FStormSyncTransportClientModule::RequestPackagesStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& InDoneDelegate) const
{
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::RequestPackagesStatus - InRemoteAddress: %ls, InPackageNames: %d", *InRemoteAddress.ToString(), InPackageNames.Num());

	if (!ClientEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::RequestPackagesStatus - Unable to get client endpoint");
		return;
	}

	ClientEndpoint->RequestStatus(InRemoteAddress, InPackageNames, InDoneDelegate);
}

void FStormSyncTransportClientModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Client.Start"),
		TEXT("Starts Storm Sync Client. Usage: [EndpointName]"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportClientModule::ExecuteStartClient),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Client.Ping"),
		TEXT("Sends a ping message on Storm Sync message bus"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportClientModule::ExecutePing),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Client.SyncPak"),
		TEXT("Synchronize a pak file on Storm Sync message bus. Usage: <PackageName>... [-name=<AvaPackageName> -version=<AvaPackageVersion>] [-description=<AvaPackageVersion>] [-author=<AvaPackageAuthor>]"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportClientModule::ExecuteSyncPak),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Client.Debug"),
		TEXT("Prints out client address enpoint id to the console"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportClientModule::ExecuteDebug),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Client.Debug.Ping"),
		TEXT("Sends status ping"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportClientModule::ExecuteDebugPing),
		ECVF_Default
	));
}

void FStormSyncTransportClientModule::UnregisterConsoleCommands()
{
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}

	ConsoleCommands.Empty();
}

void FStormSyncTransportClientModule::OnPostEngineInit()
{
	bEngineInitComplete = true;
	
	if (ClientEndpoint.IsValid())
	{
		PublishStatusPingMessage();
	}
}

void FStormSyncTransportClientModule::PublishStatusPingMessage() const
{
	const FMessageEndpointSharedPtr MessageEndpoint = GetClientMessageEndpoint();
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::PublishStatusPingMessage - Unable to send Connect Message cause Message Endpoint is invalid");
		return;
	}

	// We broadcast a message to notify others about this editor instance (this is required so that further "direct" send are received on the other end)
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::PublishStatusPingMessage - Publish Client Connect Message ...");
	FStormSyncTransportStatusPing* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportStatusPing>();
	MessageEndpoint->Publish(Message);
}

void FStormSyncTransportClientModule::OnWakeup()
{
	const FMessageEndpointSharedPtr MessageEndpoint = GetClientMessageEndpoint();
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Warning, "FStormSyncTransportClientModule::OnWakeup - Unable to get client message endpoint");
		return;
	}
	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FStormSyncTransportPingMessage>());
}

void FStormSyncTransportClientModule::ExecuteStartClient(const TArray<FString>& InArgs)
{
	StartClientEndpoint(InArgs.Num() ? InArgs[0] : DefaultClientEndpointName);	
}

void FStormSyncTransportClientModule::ExecutePing(const TArray<FString>& InArgs)
{
	const FString Argv = FString::Join(InArgs, TEXT(""));
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::ExecutePing - %ls", *Argv);

	const FMessageEndpointSharedPtr MessageEndpoint = GetClientMessageEndpoint();
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Warning, "FStormSyncTransportClientModule::ExecutePing - Unable to get client message endpoint");
		return;
	}

	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FStormSyncTransportPingMessage>());
}

FStormSyncPackageDescriptor FStormSyncTransportClientModule::CreatePackageDescriptorFromCommandLine(const FString& InArgv)
{
	FStormSyncPackageDescriptor PackageDescriptor;
	if (!FParse::Value(*InArgv, TEXT("-name="), PackageDescriptor.Name))
	{
		PackageDescriptor.Name = DefaultPakName;
		UE_LOGF(LogStormSyncClient, Display, "FStormSyncEditorModule::CreatePackageDescriptorFromCommandLine - Missing -name parameter, using default \"%ls\"", *PackageDescriptor.Name);
	}

	FParse::Value(*InArgv, TEXT("-version="), PackageDescriptor.Version);
	FParse::Value(*InArgv, TEXT("-description="), PackageDescriptor.Description);
	FParse::Value(*InArgv, TEXT("-author="), PackageDescriptor.Author);

	return PackageDescriptor;
}

void FStormSyncTransportClientModule::ExecuteSyncPak(const TArray<FString>& InArgs)
{
	const FString Argv = FString::Join(InArgs, TEXT(" "));
	UE_LOGF(LogStormSyncClient, Display, "FStormSyncTransportClientModule::ExecuteSyncPak - Argv: %ls", *Argv);

	// Parse command line.
	TArray<FName> PackageNames;
	FStormSyncCommandLineUtils::Parse(*Argv, PackageNames);

	if (PackageNames.IsEmpty())
	{
		UE_LOGF(LogStormSyncClient, Error, "FStormSyncTransportClientModule::ExecuteSyncPak - Missing at least one package name to sync.");
		return;
	}

	// Create package descriptor pulling info from command line options
	const FStormSyncPackageDescriptor PackageDescriptor = CreatePackageDescriptorFromCommandLine(Argv);

	// Sync over network now
	SynchronizePackages(PackageDescriptor, PackageNames);
}

void FStormSyncTransportClientModule::ExecuteDebug(const TArray<FString>& InArgs)
{
	const FString AddressId = GetClientEndpointMessageAddressId();
	UE_LOGF(LogStormSyncClient, Display, "StormSync.Client.Debug - EndpointId: %ls", *AddressId);
	const FString InstanceId = FApp::GetInstanceId().ToString();
	UE_LOGF(LogStormSyncClient, Display, "StormSync.Client.Debug - InstanceId: %ls", *InstanceId);
}

void FStormSyncTransportClientModule::ExecuteDebugPing(const TArray<FString>& InArgs)
{
	const FMessageEndpointSharedPtr MessageEndpoint = GetClientMessageEndpoint();
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncClient, Error, "StormSync.Client.Debug.Ping - Unable to send Connect Message cause Message Endpoint is invalid");
		return;
	}

	UE_LOGF(LogStormSyncClient, Display, "StormSync.Client.Debug.Ping - Publish Client Connect Message ...");
	FStormSyncTransportStatusPing* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportStatusPing>();
	MessageEndpoint->Publish(Message);
}

IMPLEMENT_MODULE(FStormSyncTransportClientModule, StormSyncTransportClient)

#undef LOCTEXT_NAMESPACE
