// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportClientModule.h"
#include "StormSyncPackageDescriptor.h"

class IStormSyncTransportClientLocalEndpoint;
struct FMessageAddress;

class IStormSyncTransportLocalEndpoint;

class FMessageEndpoint;

class FStormSyncTransportClientModule : public IStormSyncTransportClientModule
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//~ Begin IStormSyncTransportClientModule
	virtual void StartClientEndpoint(const FString& InEndpointFriendlyName) override;
	virtual TSharedPtr<IStormSyncTransportClientLocalEndpoint> CreateClientLocalEndpoint(const FString& InEndpointFriendlyName) const override;
	virtual FString GetClientEndpointMessageAddressId() const override;
	virtual FMessageEndpointSharedPtr GetClientMessageEndpoint() const override;
	virtual void SynchronizePackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames) const override;
	virtual void PushPackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FMessageAddress& InMessageAddress, const FOnStormSyncPushComplete& InDoneDelegate) const override;
	virtual void PullPackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FMessageAddress& InMessageAddress, const FOnStormSyncPullComplete& InDoneDelegate) const override;
	virtual void RequestPackagesStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& InDoneDelegate) const override;
	//~ End IStormSyncTransportClientModule

private:
	/** Called from StartupModule and sets up console commands for the plugin via IConsoleManager */
	void RegisterConsoleCommands();

	/** Called from ShutdownModule and clears out previously registered console commands */
	void UnregisterConsoleCommands();
	
	/** Event handler to kick in operations once engine is fully initialized (to publish a client connect message) */
	void OnPostEngineInit();

	/** Publish the client endpoint's ping message. */
	void PublishStatusPingMessage() const;

	/** Handler for wake up event. */
	void OnWakeup();

	/** Command handler for starting the client. */
	void ExecuteStartClient(const TArray<FString>& InArgs);
	
	/** Command handler for ping command */
	void ExecutePing(const TArray<FString>& InArgs);
	
	/** Command handler for sync pak command */
	void ExecuteSyncPak(const TArray<FString>& InArgs);

	/** Command handler for displaying debug info. */
	void ExecuteDebug(const TArray<FString>& InArgs);

	/** Command handler for sending a status ping. */
	void ExecuteDebugPing(const TArray<FString>& InArgs);

	/** Returns a new package descriptor pulling info from command line options */
	static FStormSyncPackageDescriptor CreatePackageDescriptorFromCommandLine(const FString& InArgv);

private:
	/** The name of the ava pak to use when none is provided from command line arguments */
	static constexpr const TCHAR* DefaultPakName = TEXT("SyncPak");

	/** Default client endpoint name. */
	static constexpr const TCHAR* DefaultClientEndpointName = TEXT("Client");
	
	/** Our message endpoint provider */
	TSharedPtr<IStormSyncTransportClientLocalEndpoint, ESPMode::ThreadSafe> ClientEndpoint;
	
	/** References of registered console commands via IConsoleManager */
	TArray<IConsoleObject*> ConsoleCommands;

	/** Indicates when the engine init is complete. This is used to know when pending messages can be published. */
	bool bEngineInitComplete = false;
};
