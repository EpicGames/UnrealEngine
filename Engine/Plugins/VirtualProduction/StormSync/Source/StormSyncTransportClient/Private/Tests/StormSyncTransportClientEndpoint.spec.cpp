// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonObjectConverter.h"
#include "MessageEndpointBuilder.h"
#include "StormSyncTransportClientEndpoint.h"
#include "StormSyncTransportNetworkUtils.h"
#include "StormSyncTransportSettings.h"
#include "StormSyncTransportTcpPackets.h"
#include "Algo/Transform.h"
#include "Common/TcpListener.h"
#include "Misc/AutomationTest.h"

/**
 * Local mock tcp server for this suite.
 *
 * Mimics some of the functionality of FStormSyncTransportTcpServer
 */
class FMockTcpServer : public FRunnable
{
public:
	FMockTcpServer()
	{
		SleepTime = FTimespan::FromSeconds(1);

		FIPv4Endpoint ServerEndpoint;
		FIPv4Endpoint::Parse(GetDefault<UStormSyncTransportSettings>()->GetServerEndpoint(), ServerEndpoint);

		// Use configured port with an offset of +1, just to ensure mock tcp server doesn't conflict with a regular server that might be running
		const uint16 ServerPort = GetDefault<UStormSyncTransportSettings>()->GetTcpServerPort();
		Endpoint = MakeUnique<FIPv4Endpoint>(FIPv4Address::InternalLoopback, ServerPort + 1);
		Thread = FRunnableThread::Create(this, TEXT("FMockTcpServer"), 128 * 1024, TPri_Normal);
	}

	~FMockTcpServer()
	{
		if (Thread != nullptr)
		{
			Thread->Kill(true);
			delete Thread;
		}

		StopListening();
	}

	DECLARE_DELEGATE_OneParam(FOnReceivedBufferDelegate, const FStormSyncBufferPtr&);
	FOnReceivedBufferDelegate OnReceivedBuffer;
	
	DECLARE_DELEGATE_OneParam(FOnReceivedConnection, const FIPv4Endpoint&);
	FOnReceivedConnection OnReceivedConnection;
	
	DECLARE_DELEGATE_OneParam(FOnReceivedSizeHeader, const uint32);
	FOnReceivedSizeHeader OnReceivedSizeHeader;

	virtual bool Init() override
	{
		check(!SocketListener);

		SocketListener = MakeUnique<FTcpListener>(*Endpoint, FTimespan::FromSeconds(1), false);
		if (SocketListener->IsActive())
		{
			SocketListener->OnConnectionAccepted().BindRaw(this, &FMockTcpServer::OnConnectionAccepted);
			UE_LOGF(LogTemp, Display, "FMockTcpServer: Started listening on %ls:%d", *SocketListener->GetLocalEndpoint().Address.ToString(), SocketListener->GetLocalEndpoint().Port);
			return true;
		}

		UE_LOGF(LogTemp, Error, "FMockTcpServer: Could not create Tcp Listener!");
		return false;	
	}

	virtual void Stop() override
	{
		bStopping = true;
	}

	void StopListening()
	{
		if (SocketListener.IsValid())
		{
			UE_LOGF(LogTemp, Display, "FMockTcpServer: No longer listening on %ls:%d", *SocketListener->GetLocalEndpoint().Address.ToString(), SocketListener->GetLocalEndpoint().Port);
			SocketListener.Reset();
		}

		if (RemoteConnection.IsValid())
		{
			RemoteConnection->Close();
			RemoteConnection.Reset();
		}
	}

	virtual uint32 Run() override
	{
		while (!bStopping)
		{
			if (RemoteConnection.IsValid())
			{
				uint32 PendingDataSize = 0;
				while (RemoteConnection->HasPendingData(PendingDataSize))
				{
					TArray<uint8> Buffer;
					Buffer.AddUninitialized(PendingDataSize);
					int32 BytesRead = 0;
					if (!RemoteConnection->Recv(Buffer.GetData(), PendingDataSize, BytesRead, ESocketReceiveFlags::None))
					{
						UE_LOGF(LogTemp, Error, "FMockTcpServer: Error while receiving data via endpoint %ls", *RemoteEndpoint.ToString());
						continue;
					}

					TArray<uint8>& MessageBuffer = ReceiveBuffer;
					for (int32 i = 0; i < BytesRead; ++i)
					{
						MessageBuffer.Add(Buffer[i]);

						// First 4 expected bytes represents the Buffer size we're gonna get
						if (!bHasReceivedSize && i == 3)
						{
							const uint32 Size = *reinterpret_cast<const uint32*>(MessageBuffer.GetData());
			
							UE_LOGF(LogTemp, Verbose, "FMockTcpServer: Received size: %d. Setting it for %ls", Size, *RemoteEndpoint.ToString());
							BufferExpectedSize = Size;
							bHasReceivedSize = true;
							MessageBuffer.Empty();

							OnReceivedSizeHeader.ExecuteIfBound(Size);
						}
					}

					if (bHasReceivedSize)
					{
						const int32 BufferSize = MessageBuffer.Num();
						
						// Send back to client the size buffer we received so far
						SendMessage(CreateMessage(FStormSyncTransportTcpSizePacket(BufferSize)));

						if (BufferSize == BufferExpectedSize)
						{
							// Send back to client transfer complete packet, so that it knows we're done with the transfer
							SendMessage(CreateMessage(FStormSyncTransportTcpTransferCompletePacket()));
							
							UE_LOGF(LogTemp, Display, "\tFMockTcpServer: We received full buffer!!");
							OnReceivedBuffer.ExecuteIfBound(MakeShared<FStormSyncBuffer>(MessageBuffer));
						}
					}
				}
			}
			
			FPlatformProcess::Sleep(SleepTime.GetSeconds());
		}

		return 0;
	}

	FString GetEndpointAddress() const
	{
		return Endpoint->ToString();
	}

	uint32 GetBufferExpectedSize() const
	{
		return BufferExpectedSize;
	}

private:
	/** The endpoint used when creating the tcp listener */
	TUniquePtr<FIPv4Endpoint> Endpoint;
	
	/** Holds the thread object. */
	FRunnableThread* Thread;
	
	/** Holds the time to sleep between checking for pending connections. */
	FTimespan SleepTime;

	/** Our instance of tcp listener, created in Init() */
	TUniquePtr<FTcpListener> SocketListener;

	/** The endpoint of the client currently connected (if any) */
	FIPv4Endpoint RemoteEndpoint;
	
	/** Remote connection (we only expect to deal with a single client) */
	TSharedPtr<FSocket> RemoteConnection;

	/** Buffer received so far from the remote connection */
	TArray<uint8> ReceiveBuffer;

	/** Flag to indicate whether we received the size "header" */
	bool bHasReceivedSize = false;

	/** Updated when remote sends us the size "header" */
	uint32 BufferExpectedSize = 0;
	
	/** Holds a flag indicating that the thread is stopping. */
	std::atomic<bool> bStopping = false;

	/** Template helper to serialize a given struct into JSON for sending over tcp socket */
	template <typename InStructType>
	FString CreateMessage(const InStructType& InStruct)
	{
		FString Message;
		const bool bMessageOk = FJsonObjectConverter::UStructToJsonObjectString(InStruct, Message);
		check(bMessageOk);
		return Message;
	}
	
	/**
	 * To use in conjunction with CreateMessage()
	 *
	 * Takes a raw string and send it on provided endpoint, if we have a current active connection for it
	 * (in ClientConnections map)
	 */
	bool SendMessage(const FString& InMessage) const
	{
		int32 BytesSent = 0;
		return RemoteConnection->Send(reinterpret_cast<uint8*>(TCHAR_TO_UTF8(*InMessage)), InMessage.Len() + 1, BytesSent);	
	}

	bool OnConnectionAccepted(FSocket* InSocket, const FIPv4Endpoint& InEndpoint)
	{
		UE_LOGF(LogTemp, Display, "FMockTcpServer: Incoming connection -> %ls", *InEndpoint.ToString());

		// Cleanup previous one if any
		if (RemoteConnection.IsValid())
		{
			RemoteConnection->Close();
			RemoteConnection.Reset();
		}
		bHasReceivedSize = false;
		BufferExpectedSize = 0;
		ReceiveBuffer.Empty();

		// Update our remote connection to the incoming one
		InSocket->SetNoDelay(true);
		RemoteConnection = MakeShareable(InSocket);
		RemoteEndpoint = InEndpoint;

		// Send a dummy state packet (not json serialized to avoid adding JSON/JSONUtilities module dependencies to this module
		// Responding here is important for client socket connection to be considered successful on FStormSyncTransportClientSocket end
		int32 BytesSent = 0;
		const FString DummyMessage = TEXT("__state_dummy_packet__");
		RemoteConnection->Send(reinterpret_cast<uint8*>(TCHAR_TO_UTF8(*DummyMessage)), DummyMessage.Len() + 1, BytesSent);

		OnReceivedConnection.ExecuteIfBound(InEndpoint);
		return true;
	}
};

BEGIN_DEFINE_SPEC(FStormSyncTransportClientEndpointSpec, "StormSync.StormSyncTransportClient.StormSyncTransportClientEndpoint", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	static constexpr const TCHAR* EndpointFriendlyName = TEXT("Test Suite ClientEndpoint (Message System)");
	TSharedPtr<FStormSyncTransportClientEndpoint> ClientEndpoint;
	TSharedPtr<FMessageEndpoint> LocalMessageEndpoint;

	/** Our instance of tcp listener wrapper */
	TUniquePtr<FMockTcpServer> MockTcpServer;

END_DEFINE_SPEC(FStormSyncTransportClientEndpointSpec)

void FStormSyncTransportClientEndpointSpec::Define()
{
	Describe(TEXT("Message System"), [this]()
	{
		BeforeEach([this]()
		{
			ClientEndpoint = MakeShared<FStormSyncTransportClientEndpoint>();
			ClientEndpoint->InitializeMessaging(EndpointFriendlyName);
			LocalMessageEndpoint = FMessageEndpoint::Builder(EndpointFriendlyName);

			MockTcpServer = MakeUnique<FMockTcpServer>();
		});

		It(TEXT("Should be running"), [this]()
		{
			TestTrue(TEXT("is running"), ClientEndpoint->IsRunning());
			const TSharedPtr<FMessageEndpoint> MessageEndpoint = ClientEndpoint->GetMessageEndpoint();
			TestTrue(TEXT("message endpoint is valid and connected"), MessageEndpoint->IsEnabled() && MessageEndpoint->IsConnected());
		});

		LatentIt(TEXT("Should send tcp buffer to local server when it receives a sync response message"), [this](const FDoneDelegate& Done)
		{
			// In this test, we simple try to send a "sync response" message with an arbitrary list of modifiers, and check in return
			// our mock server is getting the expected size as an header and getting buffer data. Test will move on when the buffer is fully received (or test will timeout)
			// where we do an additional test against the expected size mock server received and the overall size of the buffer.
			//
			// This is not really a unit test, but rather an integration test where we check the overall behavior of ClientEndpoint / ClientSocket interactions.
			
			MockTcpServer->OnReceivedConnection.BindLambda([this, Done](const FIPv4Endpoint& Endpoint)
			{
				AddInfo(FString::Printf(TEXT("Mock TCP Server Received Connection: %s"), *Endpoint.ToString()));
			});

			MockTcpServer->OnReceivedSizeHeader.BindLambda([this, Done](const uint32 Size)
			{
				AddInfo(FString::Printf(TEXT("Mock TCP Server Received Size Header: %d"), Size));
			});

			MockTcpServer->OnReceivedBuffer.BindLambda([this, Done](const FStormSyncBufferPtr& Buffer)
			{
				AddInfo(FString::Printf(TEXT("Mock TCP Server Received Buffer of size %lld"), Buffer->Num()));

				TestEqual(TEXT("Received buffer has the expected size"), Buffer->Num(), MockTcpServer->GetBufferExpectedSize());
				Done.Execute();
			});
			
			// Build sync request message that is going to be sent over the network for a specific recipient
			TUniquePtr<FStormSyncTransportPushResponse> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportPushResponse>(FGuid::NewGuid()));
			if (!Message.IsValid())
			{
				AddInfo(FString::Printf(TEXT("FStormSyncStatusMessageService::RequestStatus - Sync response message is invalid")));
				return;
			}
			
			// T_Fixture_Noise_Background is roughly 20Mb in size, so we expect a buffer with an approximate size of 100Mb
			// Right now using fixtures files from StormSyncTests plugin, this breaks encapsulation.
			const TArray<FName> PackageNames = {
				TEXT("/StormSync/Fixtures/T_Fixture_Noise_Background"),
				TEXT("/StormSync/Fixtures/T_Fixture_Noise_Background"),
				TEXT("/StormSync/Fixtures/T_Fixture_Noise_Background"),
				TEXT("/StormSync/Fixtures/T_Fixture_Noise_Background"),
				TEXT("/StormSync/Fixtures/T_Fixture_Noise_Background"),
			};
			
			Message->HostName = FStormSyncTransportNetworkUtils::GetServerName();
			Message->HostAddress = MockTcpServer->GetEndpointAddress();
			Message->HostAdapterAddresses = FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses();
			
			TArray<FStormSyncFileModifierInfo> FileModifiers;
			Algo::Transform(PackageNames, FileModifiers, [](const FName PackageName)
			{
				return FStormSyncFileModifierInfo(EStormSyncModifierOperation::Addition, FStormSyncFileDependency(PackageName));
			});
			
			Message->Modifiers = FileModifiers;

			AddInfo(FString::Printf(TEXT("Send sync response message to %s"), *ClientEndpoint->GetMessageEndpoint()->GetAddress().ToString()));
			LocalMessageEndpoint->Send(Message.Release(), ClientEndpoint->GetMessageEndpoint()->GetAddress());
		});

		AfterEach([this]()
		{
			ClientEndpoint.Reset();
			LocalMessageEndpoint->Disable();
			LocalMessageEndpoint.Reset();
			MockTcpServer.Reset();
		});
	});
}
