// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportLocalEndpoint.h"
#include "StormSyncTransportMessages.h"

struct FMessageAddress;

DECLARE_DELEGATE_OneParam(FOnStormSyncRequestStatusComplete, const TSharedPtr<FStormSyncTransportStatusResponse>&);
DECLARE_DELEGATE_OneParam(FOnStormSyncPushComplete, const TSharedPtr<FStormSyncTransportPushResponse>&);
DECLARE_DELEGATE_OneParam(FOnStormSyncPullComplete, const TSharedPtr<FStormSyncTransportPullResponse>&);

/** Local Endpoint interface for Storm Sync Client */
class IStormSyncTransportClientLocalEndpoint : public IStormSyncTransportLocalEndpoint
{
public:
	/** Request a status from the given remote address. The given delegate will be called when the response comes in */
	virtual void RequestStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& InDoneDelegate) const = 0;
	
	/** Sends a push request message to the given remote address. The given delegate will be called when the response comes in */
	virtual void RequestPushPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPushComplete& InDoneDelegate) const = 0;
	
	/** Sends a pull request message to the given remote address. The given delegate will be called when the response comes in */
	virtual void RequestPullPackages(const FMessageAddress& InRemoteAddress, const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FOnStormSyncPullComplete& InDoneDelegate) const = 0;
	
	/** Abort a request using the GUID previously obtained via a call to RequestStatus */
	virtual void AbortStatusRequest(const FGuid& InStatusRequestId) const = 0;

	/** Abort a request using the GUID previously obtained via a call to RequestPushPackages */
	virtual void AbortPushRequest(const FGuid& InPushRequestId) const = 0;
	
	/** Abort a request using the GUID previously obtained via a call to RequestPullPackages */
	virtual void AbortPullRequest(const FGuid& InPullRequestId) const = 0;
};
