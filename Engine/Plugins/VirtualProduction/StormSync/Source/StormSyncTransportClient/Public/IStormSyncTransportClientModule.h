// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportClientLocalEndpoint.h"
#include "StormSyncTransportMessages.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FMessageEndpoint;
struct FMessageAddress;

using FMessageEndpointSharedPtr = TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>;

class IStormSyncTransportClientModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IStormSyncTransportClientModule& Get()
	{
		static const FName ModuleName = "StormSyncTransportClient";
		return FModuleManager::LoadModuleChecked<IStormSyncTransportClientModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "StormSyncTransportClient";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/** Start transport client endpoint */
	virtual void StartClientEndpoint(const FString& InEndpointFriendlyName) = 0;
	
	UE_DEPRECATED(5.5, "Use StartClientEndpoint() instead.")
	virtual TSharedPtr<IStormSyncTransportClientLocalEndpoint> CreateClientLocalEndpoint(const FString& InEndpointFriendlyName) const = 0;
	
	/** Returns Message Address UID for client endpoint if it is currently running, empty string otherwise */
	virtual FString GetClientEndpointMessageAddressId() const = 0;
	
	/** Returns Message Address UID for client endpoint if it is currently running, empty string otherwise */
	virtual FMessageEndpointSharedPtr GetClientMessageEndpoint() const = 0;

	/** Broadcast a synchronization request on all storm sync connected devices */
	virtual void SynchronizePackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& PackageNames) const = 0;
	
	/** Send a synchronization (push) request on a specific storm sync connected device */
	virtual void PushPackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FMessageAddress& InMessageAddress, const FOnStormSyncPushComplete& InDoneDelegate = FOnStormSyncPushComplete()) const = 0;
	
	/** Send a synchronization (pull) request on a specific storm sync connected device */
	virtual void PullPackages(const FStormSyncPackageDescriptor& InPackageDescriptor, const TArray<FName>& InPackageNames, const FMessageAddress& InMessageAddress, const FOnStormSyncPullComplete& InDoneDelegate = FOnStormSyncPullComplete()) const = 0;

	/** Sends a status request */
	virtual void RequestPackagesStatus(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnStormSyncRequestStatusComplete& InDoneDelegate) const = 0;
};
