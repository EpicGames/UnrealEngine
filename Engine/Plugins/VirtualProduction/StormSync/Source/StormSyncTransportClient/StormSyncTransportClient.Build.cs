// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StormSyncTransportClient : ModuleRules
	{
		public StormSyncTransportClient(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"StormSyncTransportCore"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"Json",
					"JsonUtilities",
					"Networking",
					"Sockets",
					"StormSyncCore",
				}
			);
		}
	}
}
