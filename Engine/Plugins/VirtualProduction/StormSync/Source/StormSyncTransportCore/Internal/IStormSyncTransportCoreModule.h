// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IStormSyncTransportCoreModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IStormSyncTransportCoreModule& Get()
	{
		static const FName ModuleName = "StormSyncTransportCore";
		return FModuleManager::LoadModuleChecked<IStormSyncTransportCoreModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable()
	{
		static const FName ModuleName = "StormSyncTransportCore";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/** Definition of a delegate to query endpoint configuration. */
	DECLARE_DELEGATE_RetVal(FString, FOnGetEndpointConfig);

	/** Delegate for querying the currently bound tcp server address. */
	virtual FOnGetEndpointConfig& OnGetCurrentTcpServerEndpointAddress() = 0;

	/** Delegate for querying the message bus server endpoint address. */
	virtual FOnGetEndpointConfig& OnGetServerEndpointMessageAddress() = 0;

	/** Delegate for querying the message bus client endpoint address. */
	virtual FOnGetEndpointConfig& OnGetClientEndpointMessageAddress() = 0;
};
