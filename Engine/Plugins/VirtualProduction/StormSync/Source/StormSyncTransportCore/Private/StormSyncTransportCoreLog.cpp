// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportCoreLog.h"

DEFINE_LOG_CATEGORY(LogStormSyncTransportCore);
