// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportCoreModule.h"

IMPLEMENT_MODULE(FStormSyncTransportCoreModule, StormSyncTransportCore)