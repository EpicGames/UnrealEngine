// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportCoreModule.h"

class FStormSyncDiscoveryManager;

/**
 * Implements the StormSyncTransportCore module.
 */
class FStormSyncTransportCoreModule : public IStormSyncTransportCoreModule
{
public:
	//~ Begin IStormSyncTransportCoreModule
	virtual FOnGetEndpointConfig& OnGetCurrentTcpServerEndpointAddress() override { return OnGetCurrentTcpServerAddressDelegate; }
	virtual FOnGetEndpointConfig& OnGetServerEndpointMessageAddress() override { return OnGetServerEndpointMessageAddressDelegate; }
	virtual FOnGetEndpointConfig& OnGetClientEndpointMessageAddress() override { return OnGetClientEndpointMessageAddressDelegate; }
	//~ End IStormSyncTransportCoreModule

private:
	FOnGetEndpointConfig OnGetCurrentTcpServerAddressDelegate;
	FOnGetEndpointConfig OnGetServerEndpointMessageAddressDelegate;
	FOnGetEndpointConfig OnGetClientEndpointMessageAddressDelegate;
};
