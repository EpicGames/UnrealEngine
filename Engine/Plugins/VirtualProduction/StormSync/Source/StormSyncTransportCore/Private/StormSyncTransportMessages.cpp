// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportMessages.h"

#include "Engine/Engine.h"
#include "Misc/Paths.h"

FStormSyncConnectionInfo::FStormSyncConnectionInfo()
	: EngineVersion(FNetworkVersion::GetLocalNetworkVersion())
	, InstanceId(FApp::GetInstanceId())
	, SessionId(FApp::GetSessionId())
	, HostName(FPlatformProcess::ComputerName())
	, ProjectName(FApp::GetProjectName())
{
	// We only want the dirname, not absolute path
	ProjectDir = GetBasename(FPaths::ProjectDir());

	// Figure out which type of instance it is
	if (GEngine == nullptr)
	{
		InstanceType = EStormSyncEngineType::Unknown;
	}
	else if (IsRunningDedicatedServer())
	{
		InstanceType = EStormSyncEngineType::Server;
	}
	else if (IsRunningCommandlet())
	{
		InstanceType = EStormSyncEngineType::Commandlet;
	}
	else if (GEngine->IsEditor())
	{
		InstanceType = EStormSyncEngineType::Editor;
	}
	else if (IsRunningGame())
	{
		InstanceType = EStormSyncEngineType::Game;
	}
	else
	{
		InstanceType = EStormSyncEngineType::Other;
	}	
}

FString FStormSyncConnectionInfo::ToString() const
{
	return FString::Printf(
		TEXT("EngineVersion: %d, HostName: %s, InstanceId: %s, InstanceType: %s, SessionId: %s, ProjectName: %s, ProjectDir: %s, StormSyncServerAddressId: %s"),
		EngineVersion,
		*HostName,
		*InstanceId.ToString(),
		*UEnum::GetValueAsString(InstanceType),
		*SessionId.ToString(),
		*ProjectName,
		*ProjectDir,
		*StormSyncServerAddressId
	);
}

FString FStormSyncConnectionInfo::GetBasename(const FString& InPath)
{
	FString LocalPath = InPath;
	LocalPath.RemoveFromEnd(TEXT("/"));
	return FPaths::GetBaseFilename(LocalPath);
}

FString FStormSyncTransportSyncRequest::ToString() const
{
	TArray<FString> PackageNamesStr;
	for (const FName& PackageName : PackageNames)
	{
		PackageNamesStr.Add(PackageName.ToString());
	}

	return FString::Printf(TEXT("(ID: %s) PackageDescriptor: %s, PackageNames:\n\t%s"), *MessageId.ToString(), *PackageDescriptor.ToString(), *FString::Join(PackageNamesStr, TEXT("\n\t")));
}

FString FStormSyncTransportSyncResponse::ToString() const
{
	const FString RequestDebugStr = Super::ToString();
	return FString::Printf(TEXT("HostName: %s, HostAddress: %s, HostAdapterAddresses: %s, %s"), *HostName, *HostAddress, *FString::Join(HostAdapterAddresses, TEXT(", ")), *RequestDebugStr);
}

FString FStormSyncTransportPullResponse::ToString() const
{
	TArray<FString> PackageNamesStr;
	for (const FName& PackageName : PackageNames)
	{
		PackageNamesStr.Add(PackageName.ToString());
	}
	
	return FString::Printf(
		TEXT("StormSyncTransportPullResponse (Id: %s, Status: %s, StatusText: %s) - Hostname: %s, HostAddress: %s, HostAdapterAddresses: %s, PackageDescriptor: %s, PackageNames:\n\t%s"),
		*MessageId.ToString(),
		*UEnum::GetValueAsString(Status),
		*StatusText.ToString(),
		*HostName,
		*HostAddress,
		*FString::Join(HostAdapterAddresses, TEXT(", ")),
		*PackageDescriptor.ToString(),
		*FString::Join(PackageNamesStr, TEXT("\n\t"))
	);
}

FString FStormSyncTransportPingMessage::ToString() const
{
	return FString::Printf(TEXT("Hostname %s, Username %s, ProjectName: %s"), *Hostname, *Username, *ProjectName);
}
