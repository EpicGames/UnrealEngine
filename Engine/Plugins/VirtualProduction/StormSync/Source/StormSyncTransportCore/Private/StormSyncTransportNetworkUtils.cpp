// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportNetworkUtils.h"

#include "Algo/Transform.h"
#include "IStormSyncTransportCoreModule.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "StormSyncTransportCoreLog.h"
#include "StormSyncTransportSettings.h"

namespace UE::StormSyncTransportCore::Utils::Private
{
	FString ExecuteIfBound(IStormSyncTransportCoreModule::FOnGetEndpointConfig& InDelegate, const FString& InDefaultValue)
	{
		return InDelegate.IsBound() ? InDelegate.Execute() : InDefaultValue;
	}
}

FString FStormSyncTransportNetworkUtils::GetServerName()
{
	const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
	check(Settings);
	
	const FString ServerName = Settings->GetServerName();
	return ServerName.IsEmpty() ? FPlatformProcess::ComputerName() : ServerName;
}

FString FStormSyncTransportNetworkUtils::GetTcpEndpointAddress()
{
	const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
	check(Settings);
	
	const FString ServerEndpoint = Settings->GetServerEndpoint();
	
	FIPv4Endpoint Endpoint;
	if (!FIPv4Endpoint::Parse(ServerEndpoint, Endpoint))
	{
		UE_LOGF(LogStormSyncTransportCore, Error, "UStormSyncTransportServerUtils::GetTcpEndpointAddress - Failed to parse endpoint '%ls'", *ServerEndpoint);
		return TEXT("");
	}

	return Endpoint.ToString();
}

TArray<FString> FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses()
{
	TArray<FString> AdapterAddresses;

	// Request the current server endpoint address in order to extract it's current port,
	// which may be different from the configured port.
	FString ServerEndpoint = GetCurrentTcpServerEndpointAddress();

	// If the server is not currently running, fallback to the settings.
	if (ServerEndpoint.IsEmpty())
	{
		const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
		if (ensure(Settings))
		{
			ServerEndpoint = Settings->GetServerEndpoint();
		}
	}
	
	FIPv4Endpoint Endpoint;
	if (!FIPv4Endpoint::Parse(ServerEndpoint, Endpoint))
	{
		UE_LOGF(LogStormSyncTransportCore, Error, "UStormSyncTransportServerUtils::GetLocalAdapterAddresses - Failed to parse endpoint '%ls'", *ServerEndpoint);
		return AdapterAddresses;
	}
	
	TArray<TSharedPtr<FInternetAddr>> Addresses;
	ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(Addresses);

	Algo::Transform(Addresses, AdapterAddresses, [ServerPort = Endpoint.Port](const TSharedPtr<FInternetAddr>& Address)
	{
		return FString::Printf(TEXT("%s:%d"), *Address->ToString(false), ServerPort);
	});
	
	return AdapterAddresses;
}

FString FStormSyncTransportNetworkUtils::GetCurrentTcpServerEndpointAddress()
{
	using namespace UE::StormSyncTransportCore::Utils::Private;
	return ExecuteIfBound(IStormSyncTransportCoreModule::Get().OnGetCurrentTcpServerEndpointAddress(), FString());
}

FString FStormSyncTransportNetworkUtils::GetServerEndpointMessageAddress()
{
	using namespace UE::StormSyncTransportCore::Utils::Private;
	return ExecuteIfBound(IStormSyncTransportCoreModule::Get().OnGetServerEndpointMessageAddress(), FString());
}

FString FStormSyncTransportNetworkUtils::GetClientEndpointMessageAddress()
{
	using namespace UE::StormSyncTransportCore::Utils::Private;
	return ExecuteIfBound(IStormSyncTransportCoreModule::Get().OnGetClientEndpointMessageAddress(), FString());
}
