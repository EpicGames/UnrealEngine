// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportSettings.h"

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "StormSyncTransportCoreLog.h"
#include "Utils/StormSyncTransportCommandUtils.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#endif

UStormSyncTransportSettings::UStormSyncTransportSettings()
	: TcpServerAddress(TEXT("0.0.0.0"))
{
	CategoryName = TEXT("StormSync");
	SectionName = TEXT("Transport & Network");
}

const UStormSyncTransportSettings& UStormSyncTransportSettings::Get()
{
	const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
	check(Settings);
	return *Settings;
}

#if WITH_EDITOR
void UStormSyncTransportSettings::OpenEditorSettingsWindow() const
{
	static ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	SettingsModule.ShowViewer(GetContainerName(), GetCategoryName(), SectionName);
}
#endif

FString UStormSyncTransportSettings::GetServerEndpoint() const
{
	bool bReportUdpSettingError = false;
	FString ServerAddress = TcpServerAddress;
	if (!bOverrideServerAddress)
	{
		// Server override is disabled in project settings (the default behavior)
		// Use the same value as defined in UDPMessagingSettings > Unicast Endpoint
		FString UnicastPort;
		FString UnicastHostname;
		if (GetUdpMessagingUnicastEndpoint(UnicastHostname, UnicastPort))
		{
			ServerAddress = UnicastHostname;
		}
		else
		{
			// Either unable to get the setting or malformed
			bReportUdpSettingError = true;
		}
	}
	
	// Take the command line overrides if they are used and valid (eg. `-StormSyncServerEndpoint=`)
	FString CommandLineOverride;
	if (UE::StormSync::Transport::Private::GetServerEndpointParam(CommandLineOverride, ServerAddress))
	{
		return CommandLineOverride;
	}

	// Report the previous error if we end up using the address from the settings.
	if (bReportUdpSettingError)
	{
		UE_LOGF(LogStormSyncTransportCore, Display,
			"UStormSyncTransportSettings::GetServerEndpoint - Was unable to get UnicastEndpoint from UDPMessagingSettings, using \"%ls\" instead."
			, *ServerAddress);
	}

	return FString::Printf(TEXT("%s:%d"), *ServerAddress, TcpServerPort);
}

FString UStormSyncTransportSettings::GetTcpServerAddress() const
{
	return TcpServerAddress;
}

uint16 UStormSyncTransportSettings::GetTcpServerPort() const
{
	return TcpServerPort;
}

uint32 UStormSyncTransportSettings::GetInactiveTimeoutSeconds() const
{
	return InactiveTimeoutSeconds;
}

bool UStormSyncTransportSettings::IsTcpDryRun() const
{
	return bTcpDryRun;
}

uint32 UStormSyncTransportSettings::GetConnectionRetryDelay() const
{
	return ConnectionRetryDelay;
}

bool UStormSyncTransportSettings::HasConnectionRetryDelay() const
{
	return ConnectionRetryDelay > 0;
}

FString UStormSyncTransportSettings::GetServerName() const
{
	return ServerName;
}

bool UStormSyncTransportSettings::IsAutoStartServer() const
{
	return bAutoStartServer;
}

float UStormSyncTransportSettings::GetMessageBusHeartbeatPeriod() const
{
	return MessageBusHeartbeatPeriod;
}

double UStormSyncTransportSettings::GetMessageBusHeartbeatTimeout() const
{
	return MessageBusHeartbeatTimeout;
}

float UStormSyncTransportSettings::GetDiscoveryManagerTickInterval() const
{
	return DiscoveryManagerTickInterval;
}

double UStormSyncTransportSettings::GetMessageBusTimeBeforeRemovingInactiveSource() const
{
	return MessageBusTimeBeforeRemovingInactiveSource;
}

bool UStormSyncTransportSettings::IsDiscoveryPeriodicPublishEnabled() const
{
	return bEnableDiscoveryPeriodicPublish;
}

bool UStormSyncTransportSettings::ShouldShowImportWizard() const
{
	return bShowImportWizard;
}

bool UStormSyncTransportSettings::GetUdpMessagingUnicastEndpoint(FString& OutUnicastHostname, FString& OutUnicastPort) const
{
	const FConfigFile* EngineConfig = GConfig ? GConfig->FindConfigFileWithBaseName(TEXT("Engine")) : nullptr;
	if (!EngineConfig)
	{
		return false;
	}
	
	// Unicast endpoint setting
	FString UnicastEndpoint;
	if (!EngineConfig->GetString(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("UnicastEndpoint"), UnicastEndpoint) || UnicastEndpoint.IsEmpty())
	{
		return false;
	}
	
	// Parse value overrides (if present)
	FParse::Value(FCommandLine::Get(), TEXT("-UDPMESSAGING_TRANSPORT_UNICAST="), UnicastEndpoint);

	TArray<FString> Settings;
	if (UnicastEndpoint.ParseIntoArray(Settings, TEXT(":"), false) == 2)
	{
		OutUnicastHostname = Settings[0];
		OutUnicastPort = Settings[1];
		return true;
	}
	
	return false;
}
