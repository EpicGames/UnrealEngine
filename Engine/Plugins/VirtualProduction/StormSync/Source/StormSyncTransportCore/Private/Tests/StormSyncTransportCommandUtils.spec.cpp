// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Misc/CommandLine.h"
#include "StormSyncTransportSettings.h"
#include "Utils/StormSyncTransportCommandUtils.h"

BEGIN_DEFINE_SPEC(FStormSyncTransportCommandUtilsSpec, "StormSync.StormSyncTransportCore.StormSyncTransportCommandUtils", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	FString OriginalCmdLine;
END_DEFINE_SPEC(FStormSyncTransportCommandUtilsSpec)

void FStormSyncTransportCommandUtilsSpec::Define()
{
	Describe(TEXT("StormSyncServerUtils"), [this]()
	{
		using namespace UE::StormSync::Transport::Private;
		BeforeEach([this]()
		{
			OriginalCmdLine = FCommandLine::Get();
		});

		It(TEXT("IsServerAutoStartDisabled()"), [this]()
		{
			FCommandLine::Set(TEXT("..."));
			TestFalse(TEXT("IsServerAutoStarted() returns false with no command line flag"), IsServerAutoStartDisabled());

			FCommandLine::Set(TEXT("... -NoStormSyncServerAutoStart"));
			TestTrue(TEXT("IsServerAutoStarted() returns true with -NoStormSyncServerAutoStart flag"), IsServerAutoStartDisabled());

			FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=1234"));
			TestFalse(TEXT("IsServerAutoStarted() returns false with -StormSyncServerEndpoint=1234 flag"), IsServerAutoStartDisabled());

			FCommandLine::Set(TEXT("... -NoStormSyncServerAutoStart -StormSyncServerEndpoint=1234"));
			TestTrue(TEXT("IsServerAutoStarted() returns true with both flag"), IsServerAutoStartDisabled());
		});

		It(TEXT("UStormSyncTransportSettings::GetServerEndpoint() should use CLI overrides when present"), [this]()
		{
			const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
			check(Settings);

			const FString DefaultHostname = Settings->GetTcpServerAddress();
			const uint16 DefaultPort = Settings->GetTcpServerPort();
			const FString DefaultServerEndpoint = Settings->GetServerEndpoint();

			// When command line has no -StormSyncServerEndpoint flag, returns the settings value
			FCommandLine::Set(TEXT("..."));
			TestEqual(TEXT("DefaultServerEndpoint with no flags"), DefaultServerEndpoint, FString::Printf(TEXT("%s:%d"), *DefaultHostname, DefaultPort));

			// With override, returns the CLI flag value
			{
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=1234"));

				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestEqual(TEXT("GetServerEndpointParam() returns correct ServerEndpoint"), ServerEndpoint, FString::Printf(TEXT("%s:%d"), *DefaultHostname, 1234));
				TestEqual(TEXT("UStormSyncTransportSettings::GetServerEndpoint() returns correct ServerEndpoint"), Settings->GetServerEndpoint(), ServerEndpoint);
				TestEqual(TEXT("UStormSyncTransportSettings::GetServerEndpoint() returns correct ServerEndpoint"), Settings->GetServerEndpoint(), FString::Printf(TEXT("%s:%d"), *DefaultHostname, 1234));
			}

			{
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=192.0.0.110:1234"));

				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestTrue(TEXT("GetServerEndpointParam() returns true with -StormSyncServerEndpoint=192.0.0.110 flag"), bValidEndpoint);
				TestEqual(TEXT("GetServerEndpointParam() returns correct ServerEndpoint"), ServerEndpoint, TEXT("192.0.0.110:1234"));
				TestEqual(TEXT("UStormSyncTransportSettings::GetServerEndpoint() returns correct ServerEndpoint"), Settings->GetServerEndpoint(), ServerEndpoint);
				TestEqual(TEXT("UStormSyncTransportSettings::GetServerEndpoint() returns correct ServerEndpoint"), Settings->GetServerEndpoint(), TEXT("192.0.0.110:1234"));
			}
		});

		It(TEXT("GetServerEndpointParam()"), [this]()
		{
			const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
			check(Settings);

			const FString DefaultHostname = Settings->GetTcpServerAddress();
			
			{
				FCommandLine::Set(TEXT("... "));
				
				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestFalse(TEXT("GetServerEndpointParam() returns false with no command line flag"), bValidEndpoint);
				TestTrue(TEXT("GetServerEndpointParam() returns empty ServerEndpoint"), ServerEndpoint.IsEmpty());
			}

			{
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=1234"));
				
				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestTrue(TEXT("GetServerEndpointParam() returns true with -StormSyncServerEndpoint=1234 flag"), bValidEndpoint);
				TestEqual(TEXT("GetServerEndpointParam() returns correct ServerEndpoint"), ServerEndpoint, FString::Printf(TEXT("%s:%d"), *DefaultHostname, 1234));
			}

			{
				// given invalid port
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=1ze2"));

				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestFalse(TEXT("GetServerEndpointParam() returns false with -StormSyncServerEndpoint=1ze2 flag"), bValidEndpoint);
				TestTrue(TEXT("GetServerEndpointParam() returns empty ServerEndpoint"), ServerEndpoint.IsEmpty());
			}

			{
				// missing port part
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=localhost"));

				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestFalse(TEXT("GetServerEndpointParam() returns false with -StormSyncServerEndpoint=localhost flag"), bValidEndpoint);
				TestTrue(TEXT("GetServerEndpointParam() returns empty ServerEndpoint"), ServerEndpoint.IsEmpty());
			}

			{
				// port not a valid int
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=localhost:1ze2"));

				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestFalse(TEXT("GetServerEndpointParam() returns false with -StormSyncServerEndpoint=localhost::1ze2 flag"), bValidEndpoint);
				TestTrue(TEXT("GetServerEndpointParam() returns empty ServerEndpoint"), ServerEndpoint.IsEmpty());
			}

			{
				// port not a valid int
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=localhost:foo"));

				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestFalse(TEXT("GetServerEndpointParam() returns false with -StormSyncServerEndpoint=localhost::foo flag"), bValidEndpoint);
				TestTrue(TEXT("GetServerEndpointParam() returns empty ServerEndpoint"), ServerEndpoint.IsEmpty());
			}

			{
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=localhost:1234"));

				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestTrue(TEXT("GetServerEndpointParam() returns true with -StormSyncServerEndpoint=localhost:1234 flag"), bValidEndpoint);
				TestEqual(TEXT("GetServerEndpointParam() returns correct ServerEndpoint"), ServerEndpoint, TEXT("localhost:1234"));
			}

			{
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=0.0.0.0:1234"));
				
				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestTrue(TEXT("GetServerEndpointParam() returns true with -StormSyncServerEndpoint=0.0.0.0:1234 flag"), bValidEndpoint);
				TestEqual(TEXT("GetServerEndpointParam() returns correct ServerEndpoint"), ServerEndpoint, TEXT("0.0.0.0:1234"));
			}

			{
				FCommandLine::Set(TEXT("... -StormSyncServerEndpoint=192.0.0.110:1234"));
				
				FString ServerEndpoint;
				const bool bValidEndpoint = GetServerEndpointParam(ServerEndpoint);
				AddInfo(FString::Printf(TEXT("Result - ServerEndpoint: %s, bValidEndpoint: %s"), *ServerEndpoint, bValidEndpoint ? TEXT("true") : TEXT("false")));
				TestTrue(TEXT("GetServerEndpointParam() returns true with -StormSyncServerEndpoint=192.0.0.110 flag"), bValidEndpoint);
				TestEqual(TEXT("GetServerEndpointParam() returns correct ServerEndpoint"), ServerEndpoint, TEXT("192.0.0.110:1234"));
			}
		});

		AfterEach([this]()
		{
			FCommandLine::Set(*OriginalCmdLine);
		});
	});
}
