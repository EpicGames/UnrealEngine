// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageEndpoint.h"
#include "Misc/AutomationTest.h"
#include "Misc/Paths.h"
#include "StormSyncTransportMessages.h"
#include "Engine/Engine.h"

BEGIN_DEFINE_SPEC(FStormSyncTransportMessagesSpec, "StormSync.StormSyncTransportCore.StormSyncTransportMessages", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

END_DEFINE_SPEC(FStormSyncTransportMessagesSpec)

void FStormSyncTransportMessagesSpec::Define()
{
	Describe(TEXT("StormSyncTransportConnectMessage"), [this]()
	{
		It(TEXT("has correct instance type"), [this]()
		{
			const FStormSyncTransportConnectMessage ConnectMessage;

			const EStormSyncEngineType ExpectedType = GEngine == nullptr ? EStormSyncEngineType::Unknown :
				GEngine->IsEditor() ? EStormSyncEngineType::Editor :
				IsRunningGame() ? EStormSyncEngineType::Game :
				EStormSyncEngineType::Other;
			
			TestEqual(TEXT("InstanceType"), ConnectMessage.InstanceType, ExpectedType);
		});
		
		It(TEXT("has correct project dir"), [this]()
		{
			const FStormSyncTransportConnectMessage ConnectMessage;

			FString ProjectDir = FPaths::ProjectDir();
			ProjectDir.RemoveFromEnd(TEXT("/"));
			TestEqual(TEXT("ProjectDir"), ConnectMessage.ProjectDir, *FPaths::GetBaseFilename(ProjectDir, true));
		});

		It(TEXT("has other expected variables"), [this]()
		{
			const FStormSyncTransportConnectMessage ConnectMessage;
			TestEqual(TEXT("EngineVersion"), ConnectMessage.EngineVersion, FNetworkVersion::GetLocalNetworkVersion());
			TestEqual(TEXT("InstanceId"), ConnectMessage.InstanceId, FApp::GetInstanceId());
			TestEqual(TEXT("SessionId"), ConnectMessage.SessionId, FApp::GetSessionId());
			TestEqual(TEXT("HostName"), ConnectMessage.HostName, FPlatformProcess::ComputerName());
			TestEqual(TEXT("ProjectName"), ConnectMessage.ProjectName, FApp::GetProjectName());
		});

		Describe(TEXT("GetBasename"), [this]()
		{
			It(TEXT("returns the last portion of a path"), [this]()
			{
				TestEqual(TEXT("is correct value"), FStormSyncTransportConnectMessage::GetBasename(TEXT("foo/bar/baz")), TEXT("baz"));
				TestEqual(TEXT("deals with trailing slash"), FStormSyncTransportConnectMessage::GetBasename(TEXT("foo/bar/baz/")), TEXT("baz"));
				TestEqual(TEXT("deals with leading slash"), FStormSyncTransportConnectMessage::GetBasename(TEXT("/foo/bar/baz/")), TEXT("baz"));
			});
		});
	});
}
