// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/AutomationTest.h"
#include "StormSyncTransportNetworkUtils.h"
#include "StormSyncTransportSettings.h"

BEGIN_DEFINE_SPEC(FStormSyncTransportNetworkUtilsSpec, "StormSync.StormSyncTransportCore.StormSyncTransportNetworkUtils", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

END_DEFINE_SPEC(FStormSyncTransportNetworkUtilsSpec)

void FStormSyncTransportNetworkUtilsSpec::Define()
{
	Describe(TEXT("GetServerName"), [this]()
	{
		It(TEXT("should return the configured server name in settings if it's not empty, computer name otherwise"), [this]()
		{
			const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
			const FString ServerName = Settings->GetServerName();
			const FString ComputerName = FPlatformProcess::ComputerName();

			AddInfo(FString::Printf(TEXT("Server Name Setting: %s"), *ServerName));
			AddInfo(FString::Printf(TEXT("ComputerName: %s"), *ComputerName));

			const FString Result = FStormSyncTransportNetworkUtils::GetServerName();
			AddInfo(FString::Printf(TEXT("GetServerName() => %s"), *ComputerName));
			
			if (ServerName.IsEmpty())
			{
				TestEqual(TEXT("should return the computer name if server name settings is empty"), Result, ComputerName);
			}
			else
			{
				TestEqual(TEXT("should return the server name in settings if it's not empty"), Result, ServerName);
			}
		});
	});

	Describe(TEXT("GetTcpEndpointAddress"), [this]()
	{
		It(TEXT("Should return server endpoint settings"), [this]()
		{
			const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
			const FString ServerEndpoint = Settings->GetServerEndpoint();

			const FString Result = FStormSyncTransportNetworkUtils::GetTcpEndpointAddress();
			TestEqual(TEXT("should return the server endpoint settings"), Result, ServerEndpoint);

			FIPv4Endpoint Endpoint;
			FIPv4Endpoint::Parse(ServerEndpoint, Endpoint);
			TestEqual(TEXT("should return the same as IPv4Endpoint parsed address"), Result, Endpoint.ToString());
		});
	});

	Describe(TEXT("GetLocalAdapterAddresses"), [this]()
	{
		It(TEXT("should return the list of local adapter addresses as returned by the socket subsystem"), [this]()
		{
			// Get port as configured in project settings for server endpoint
			const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
			const FString ServerEndpoint = Settings->GetServerEndpoint();

			// Parse it so that we can get the port part
			FIPv4Endpoint Endpoint;
			if (!FIPv4Endpoint::Parse(ServerEndpoint, Endpoint))
			{
				AddError(FString::Printf(TEXT("GetLocalAdapterAddresses - Failed to parse endpoint '%s'"), *ServerEndpoint));
				return;
			}

			// Actual test
			TArray<TSharedPtr<FInternetAddr>> Addresses;
			ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(Addresses);

			TArray<FString> AdapterAddresses = FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses();
			for (int32 i = 0; i < AdapterAddresses.Num(); ++i)
			{
				FString Result = AdapterAddresses[i];
				FString Expected = Addresses.IsValidIndex(i) ? Addresses[i]->ToString(false) : TEXT("");
				TestEqual(
					TEXT("should match socket subsystem returned local adapter addresses"),
					Result,
					FString::Printf(TEXT("%s:%d"), *Expected, Endpoint.Port)
				);
			}
		});
	});
}
