// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/StormSyncTransportCommandUtils.h"

#include "Misc/CommandLine.h"
#include "Misc/Parse.h"
#include "StormSyncTransportSettings.h"

bool UE::StormSync::Transport::Private::GetServerEndpointParam(FString& OutEndpointValue, const FString& InDefaultHostName)
{
	FString ServerEndpoint;
	if (!FParse::Value(FCommandLine::Get(), ServerEndpointFlag, ServerEndpoint))
	{
		return false;
	}

	FString Host;
	FString Port;
	if (!ServerEndpoint.Split(TEXT(":"), &Host, &Port, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		if (!ServerEndpoint.IsNumeric())
		{
			return false;
		}

		// Failed to parse, most likely given just the port
		uint16 ServerPort;
		if (FParse::Value(FCommandLine::Get(), ServerEndpointFlag, ServerPort))
		{
			const FString DefaultHostname = !InDefaultHostName.IsEmpty() ? InDefaultHostName : GetDefault<UStormSyncTransportSettings>()->GetTcpServerAddress();
			OutEndpointValue = FString::Printf(TEXT("%s:%d"), *DefaultHostname, ServerPort);
			return true;
		}
	}

	Host.TrimStartAndEndInline();
	Port.TrimStartAndEndInline();
	if (Host.IsEmpty() || Port.IsEmpty())
	{
		return false;
	}

	if (!Port.IsNumeric())
	{
		return false;
	}

	OutEndpointValue = FString::Printf(TEXT("%s:%s"), *Host, *Port);
	return true;
}

bool UE::StormSync::Transport::Private::IsServerAutoStartDisabled()
{
	return FParse::Param(FCommandLine::Get(), NoServerAutoStartFlag);
}
