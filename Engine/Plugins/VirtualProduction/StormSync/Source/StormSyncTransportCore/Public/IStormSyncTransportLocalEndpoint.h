// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FMessageEndpoint;

/**
 * Interface representing a local endpoint you can send from for Storm Sync
 */
class IStormSyncTransportLocalEndpoint : public TSharedFromThis<IStormSyncTransportLocalEndpoint, ESPMode::ThreadSafe>
{
public:
	/** Virtual destructor */
	virtual ~IStormSyncTransportLocalEndpoint() = default;

	/** Returns whether message endpoint is currently active */
	virtual bool IsRunning() const = 0;

	/** Returns underlying message endpoint implemented by the local endpoint provider */
	virtual TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const = 0;
};
