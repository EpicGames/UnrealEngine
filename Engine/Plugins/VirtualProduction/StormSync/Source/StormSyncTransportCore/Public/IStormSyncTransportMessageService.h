// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FMessageEndpointBuilder;

/**
 * Interface for message services
 *
 * They are typically created within a IStormSyncTransportLocalEndpoint derived class, from local endpoints' InitializeMessaging() post constructor time.
 */
class IStormSyncTransportMessageService : public TSharedFromThis<IStormSyncTransportMessageService, ESPMode::ThreadSafe>
{
public:
	/** Virtual destructor */
	virtual ~IStormSyncTransportMessageService() = default;
	
	/** This lets child classes the opportunity to add custom message handlers to the endpoint builder */
	virtual void InitializeMessageEndpoint(FMessageEndpointBuilder& InEndpointBuilder) = 0;
};
