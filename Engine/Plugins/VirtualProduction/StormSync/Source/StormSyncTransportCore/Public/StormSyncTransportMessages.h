// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/App.h"
#include "StormSyncCommonTypes.h"
#include "StormSyncPackageDescriptor.h"
#include "StormSyncTransportMessages.generated.h"

/** Empty message indicating an Heartbeat event */
USTRUCT()
struct FStormSyncTransportHeartbeatMessage
{
	GENERATED_BODY()

	/** Whether local server endpoint is currently running and able to receive incoming connections */
	UPROPERTY(EditAnywhere, Category="Storm Sync Message")
	bool bIsServerRunning = false;

	/** Default constructor */
	FStormSyncTransportHeartbeatMessage() = default;
};

/** Holds data about local project (such as Message Address Ids, Project Name, Hostname, InstanceId, etc.) */
USTRUCT()
struct FStormSyncConnectionInfo
{
	GENERATED_BODY()

	/** Holds the engine version checksum */
	UPROPERTY(EditAnywhere, Category="Storm Sync Message")
	int32 EngineVersion = 0;

	/** Holds the instance identifier */
	UPROPERTY(EditAnywhere, Category="Storm Sync Message")
	FGuid InstanceId;

	/** Holds the type of the engine instance */
	UPROPERTY(EditAnywhere, Category="Storm Sync Message")
	EStormSyncEngineType InstanceType = EStormSyncEngineType::Unknown;

	/** Holds the identifier of the session that the application belongs to */
	UPROPERTY(EditAnywhere, Category="Storm Sync Message")
	FGuid SessionId;
	
	/** Holds the message bus address identifier for Storm Sync Server endpoint */
	UPROPERTY(EditAnywhere, Category="Storm Sync Message")
	FString StormSyncServerAddressId;
	
	/** Holds the message bus address identifier for Storm Sync Client endpoint */
	UPROPERTY(EditAnywhere, Category="Storm Sync Message")
	FString StormSyncClientAddressId;
	
	/** The hostname this message was generated from */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString HostName;

	/** The unreal project name this message was generated from */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString ProjectName;
	
	/** The unreal project directory this message was generated from */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString ProjectDir;

	/** Default constructor */
	STORMSYNCTRANSPORTCORE_API FStormSyncConnectionInfo();

	/** Returns debug string for this message */
	STORMSYNCTRANSPORTCORE_API FString ToString() const;

	/** Internal helper to return the last portion of a path, similar to the Unix basename command. Trailing directory separators are ignored */
	STORMSYNCTRANSPORTCORE_API static FString GetBasename(const FString& InPath);
};

/** Connection message indicating a Connect event with data about local project / application */
USTRUCT()
struct FStormSyncTransportConnectMessage : public FStormSyncConnectionInfo
{
	GENERATED_BODY()
};

/** Request sent by the discovery service to wake up the other endpoints in case they become inactive. */
USTRUCT()
struct FStormSyncTransportWakeupRequest
{
	GENERATED_BODY()
};

/**
 * FStormSyncTransportSyncRequest.
 *
 * A sync request represents the ava package and files we want to synchronize.
 *
 * Contains info such as:
 *
 * - The top level list of package names we want to synchronize
 * - The package descriptor metadata (name, description, etc.) along the list of inner dependencies
 *   and their state (file size, timestamp, file hash, etc.)
 */
USTRUCT()
struct FStormSyncTransportSyncRequest
{
	GENERATED_BODY()
	
	/** The message ID of this sync request / response transaction (used to identify the correct response from the target) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FGuid MessageId;

	/** The top level list of package names (from which we gather the list of inner file dependencies / references) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	TArray<FName> PackageNames;

	/** The package descriptor describing the state of the ava package we want to synchronize */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FStormSyncPackageDescriptor PackageDescriptor;

	/** Default constructor. */
	FStormSyncTransportSyncRequest() = default;

	explicit FStormSyncTransportSyncRequest(const FGuid& InMessageId)
		: MessageId(InMessageId)
	{
	}

	FStormSyncTransportSyncRequest(const TArray<FName>& InPackageNames, const FStormSyncPackageDescriptor& InPackageDescriptor)
		: MessageId(FGuid::NewGuid())
		, PackageNames( InPackageNames)
		, PackageDescriptor(InPackageDescriptor)
	{
	}

	STORMSYNCTRANSPORTCORE_API FString ToString() const;
};

/**
 * Push Request message (which is effectively a SyncRequest without extra info)
 *
 * This is a separate struct to differentiate this message from sync request, that is used to broadcast a
 * synchronization request to all receivers active on the network.
 *
 * This one is meant to be used to send a synchronization request to a specific remote.
 */
USTRUCT()
struct FStormSyncTransportPushRequest : public FStormSyncTransportSyncRequest
{
	GENERATED_BODY()
	
	/** Default constructor. */
	FStormSyncTransportPushRequest() = default;
	
	FStormSyncTransportPushRequest(const TArray<FName>& InPackageNames, const FStormSyncPackageDescriptor& InPackageDescriptor)
		: FStormSyncTransportSyncRequest(InPackageNames, InPackageDescriptor)
	{
	}
};

/**
 * Pull Request message (which is effectively a SyncRequest with extra info about local server and network config)
 *
 * We are sending network / server info so that remote can initiate a sync response with correct address the tcp client
 * will attempt to send buffer to.
 */
USTRUCT()
struct FStormSyncTransportPullRequest : public FStormSyncTransportSyncRequest
{
	GENERATED_BODY()

	/** The hostname this message was generated from */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString HostName;

	/** The host endpoint address local tcp server is currently listening on this message was generated from (can be 0.0.0.0) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString HostAddress;

	/**
	 * The host local addresses associated with the adapters on the local computer.
	 *
	 * Clients should try to connect to each when server is listening on 0.0.0.0
	 */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	TArray<FString> HostAdapterAddresses;
	
	/** Default constructor. */
	FStormSyncTransportPullRequest() = default;
	
	FStormSyncTransportPullRequest(const TArray<FName>& InPackageNames, const FStormSyncPackageDescriptor& InPackageDescriptor)
		: FStormSyncTransportSyncRequest(InPackageNames, InPackageDescriptor)
	{
	}
};

UENUM()
enum class EStormSyncResponseResult : uint8
{
	/** Indicates an Error happened */
	Error,

	/** Indicates successful completion */
	Success,

	/** Should not happen **/
	Unknown
};

/**
 * FStormSyncTransportSyncResponse.
 *
 * Effectively a sync request with:
 *
 * - An additional list of sync modifiers, representing the diffing results
 * and files we want to synchronize.
 * - Additional information about server network settings such as Hostname and Address.
 *
 * @see FStormSyncFileModifierInfo
 */
USTRUCT()
struct FStormSyncTransportSyncResponse : public FStormSyncTransportSyncRequest
{
	GENERATED_BODY()
	
	/** Status of the response, error or success */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	EStormSyncResponseResult Status = EStormSyncResponseResult::Unknown;

	/**
	 * Localized text that can be filled with further information about response.
	 * 
	 * In case of error, a localized text describing what went wrong.
	 *
	 * In case of success, either empty or filled with useful info for the remote to know.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FText StatusText;

	// TODO: HostName / HostAddress in this struct might be redundant with ConnectionInfo
	/** Holds data about local project and send to remote for it to display as result status */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FStormSyncConnectionInfo ConnectionInfo;

	/** List of sync modifiers, either an Addition, Missing or Overwrite operation */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	TArray<FStormSyncFileModifierInfo> Modifiers;
	
	/** The hostname this message was generated from */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString HostName;

	/** The host endpoint address local tcp server is currently listening on this message was generated from (can be 0.0.0.0) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString HostAddress;

	/**
	 * The host local addresses associated with the adapters on the local computer.
	 *
	 * Clients should try to connect to each when server is listening on 0.0.0.0
	 */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	TArray<FString> HostAdapterAddresses;

	/** Default constructor */
	FStormSyncTransportSyncResponse() = default;

	explicit FStormSyncTransportSyncResponse(const FGuid& InMessageId)
		: FStormSyncTransportSyncRequest(InMessageId)
	{
	}

	STORMSYNCTRANSPORTCORE_API FString ToString() const;
};

/**
 * FStormSyncTransportPushResponse, a FStormSyncTransportSyncResponse child struct.
 */
USTRUCT()
struct FStormSyncTransportPushResponse : public FStormSyncTransportSyncResponse
{
	GENERATED_BODY()

	/** Default constructor. */
	FStormSyncTransportPushResponse() = default;

	explicit FStormSyncTransportPushResponse(const FGuid& InMessageId)
		: FStormSyncTransportSyncResponse(InMessageId)
	{
	}
};

/**
 * FStormSyncTransportPullResponse, a FStormSyncTransportSyncResponse child struct.
 */
USTRUCT()
struct FStormSyncTransportPullResponse : public FStormSyncTransportSyncResponse
{
	GENERATED_BODY()

	/** Default constructor. */
	FStormSyncTransportPullResponse() = default;

	explicit FStormSyncTransportPullResponse(const FGuid& InMessageId)
		: FStormSyncTransportSyncResponse(InMessageId)
	{
	}
	
	STORMSYNCTRANSPORTCORE_API FString ToString() const;
};

USTRUCT()
struct FStormSyncTransportStatusRequest
{
	GENERATED_BODY()

	/** The request ID of this status (used to identify the correct response from the target) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FGuid MessageId;
	
	/** The top level list of package names (from which we gather the list of inner file dependencies / references) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	TArray<FName> PackageNames;

	/** The inner list of file assets dependencies for this ava package */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	TArray<FStormSyncFileDependency> Dependencies;
	
	FStormSyncTransportStatusRequest() = default;

	FStormSyncTransportStatusRequest(const TArray<FName>& InPackageNames, const TArray<FStormSyncFileDependency>& InFileDependencies)
		: MessageId(FGuid::NewGuid())
		, PackageNames(InPackageNames)
		, Dependencies(InFileDependencies)
	{
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("PackageNames: %d, Dependencies: %d (RequestId: %s)"), PackageNames.Num(), Dependencies.Num(), *MessageId.ToString());
	}
};

USTRUCT()
struct FStormSyncTransportStatusResponse
{
	GENERATED_BODY()
	
	/** The request ID of this status (used to identify the correct response from the target) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FGuid StatusRequestId;

	/** Holds data about local project and send to remote for it to display as result status */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FStormSyncConnectionInfo ConnectionInfo;

	/** Whether the two remotes have differing state (eg. Modifiers array is not empty) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	bool bNeedsSynchronization = false;

	/** List of sync modifiers, either an Addition, Missing or Overwrite operation */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	TArray<FStormSyncFileModifierInfo> Modifiers;
	
	FStormSyncTransportStatusResponse() = default;

	explicit FStormSyncTransportStatusResponse(const FGuid& InId)
		: StatusRequestId(InId)
	{
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("bNeedsSynchronization: %s, Modifiers: %d (RequestId: %s)"), bNeedsSynchronization ? TEXT("true") : TEXT("false"), Modifiers.Num(), *StatusRequestId.ToString());
	}
};

USTRUCT()
struct FStormSyncTransportStatusPing
{
	GENERATED_BODY()
};

USTRUCT()
struct FStormSyncTransportStatusPong
{
	GENERATED_BODY()
};

/** Basic Ping message */
USTRUCT()
struct FStormSyncTransportPingMessage
{
	GENERATED_BODY()

	/** The hostname this message was generated from (defaults to platform computer name) */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString Hostname;

	/** The platform username for the host this message was generated from */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString Username;

	/** The local project name this message was generated from */
	UPROPERTY(VisibleAnywhere, Category = "Storm Sync Message")
	FString ProjectName;

	/** Default constructor. */
	FStormSyncTransportPingMessage()
		: Hostname(FPlatformProcess::ComputerName()),
		  Username(FPlatformProcess::UserName()),
		  ProjectName(FApp::GetProjectName())
	{
	}

	FStormSyncTransportPingMessage(const FString& Hostname, const FString& Username)
		: Hostname(Hostname),
		  Username(Username),
		  ProjectName(FApp::GetProjectName())
	{
	}

	STORMSYNCTRANSPORTCORE_API FString ToString() const;
};

/** Basic Pong message (which is effectively a Ping without extra info) */
USTRUCT()
struct FStormSyncTransportPongMessage : public FStormSyncTransportPingMessage
{
	GENERATED_BODY()

	/** Default constructor. */
	FStormSyncTransportPongMessage() = default;

	FStormSyncTransportPongMessage(const FString& Hostname, const FString& Username)
		: FStormSyncTransportPingMessage(Hostname, Username)
	{
	}
};
