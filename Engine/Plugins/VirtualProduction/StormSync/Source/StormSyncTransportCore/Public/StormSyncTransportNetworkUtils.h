// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

class FString;

/**
 * Network related helpers
 *
 * Main purpose of this class is to return correct information about network settings from UStormSyncTransportSettings
 * and Socket Subsystem.
 */
class STORMSYNCTRANSPORTCORE_API FStormSyncTransportNetworkUtils
{
public:
	/** Returns either the configured Server Name in settings, or falls back to platform computer name */
	static FString GetServerName();
	
	/** Parses and return configured server endpoint in settings, or an empty string if setting is invalid  */
	static FString GetTcpEndpointAddress();
	
	/**
	 * Returns the list of local adapter addresses as returned by the socket subsystem, with current port from server
	 * endpoint appended to the address list.
	 */
	static TArray<FString> GetLocalAdapterAddresses();

	/**
	 * Returns the current tcp server endpoint address (ip:port).
	 * @remark This is potentially different from the configured endpoint address in case of port collisions, it may be on a different one.
	 */
	static FString GetCurrentTcpServerEndpointAddress();

	/** Returns the current message bus server endpoint address. */
	static FString GetServerEndpointMessageAddress();

	/** Returns the current message bus client endpoint address. */
	static FString GetClientEndpointMessageAddress();
};
