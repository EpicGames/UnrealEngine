// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "StormSyncTransportSettings.generated.h"

class FStormSyncTransportSettingsDetailsCustomization;

/**
 * Settings for the StormSyncTransport plugins.
 *
 * Handles config for transport / network related features.
 */
UCLASS(Config=Game, DefaultConfig, DisplayName="Transport & Network")
class STORMSYNCTRANSPORTCORE_API UStormSyncTransportSettings : public UDeveloperSettings
{
	GENERATED_BODY()

	/* For TcpServerAddress customization as ip dropdown */
	friend FStormSyncTransportSettingsDetailsCustomization;

public:
	UStormSyncTransportSettings();
	
	static const UStormSyncTransportSettings& Get();

#if WITH_EDITOR
	/** Helper to bring focus on transport settings in editor */
	void OpenEditorSettingsWindow() const;
#endif

	/**
	 * Gets the listen endpoint endpoint to listen to when starting off the tcp socket server listener.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 *
	 * 0.0.0.0 will bind to the default network adapter on Windows, and all available network adapters on other operating systems.
	 */
	FString GetServerEndpoint() const;

	/** Returns the hostname tcp server is configured to run on */
	FString GetTcpServerAddress() const;

	/** Returns the port tcp server is configured to run on */
	uint16 GetTcpServerPort() const;

	/**
	 * Returns th port offset range when trying to acquire an available socket for the Tcp server.
	 */
	uint16 GetTcpServerPortMaxOffset() const { return TcpServerPortMaxOffset; }

	/** Returns inactive timeout period. The amount of time in seconds to consider a client to be inactive */
	uint32 GetInactiveTimeoutSeconds() const;

	/** Returns true if we are in debug mode dry mode and shouldn't extract incoming buffer to local project */
	bool IsTcpDryRun() const;

	/** Returns connection delay */
	uint32 GetConnectionRetryDelay() const;

	/** Returns connection delay */
	bool HasConnectionRetryDelay() const;

	/** Returns user friendly name to describe this server on the network (defaults to hostname if left to empty) */
	FString GetServerName() const;

	/** Whether we should start the server endpoint on startup */
	bool IsAutoStartServer() const;

	/** Returns message bus heartbeat frequency in seconds */
	float GetMessageBusHeartbeatPeriod() const;

	/** Returns message bus heartbeat frequency in seconds, tick interval for the discovery manager to check for heartbeats timeouts */
	float GetDiscoveryManagerTickInterval() const;

	/** Returns message bus heartbeat timeout in seconds, to determine when a connected device becomes unresponsive */
	double GetMessageBusHeartbeatTimeout() const;

	/** Returns message bus inactive timeout in seconds, to determine when a connected device is considered disconnected */
	double GetMessageBusTimeBeforeRemovingInactiveSource() const;

	/** Returns whether discovery manager should send a connect message at a regular interval (broadcast / publish) */
	bool IsDiscoveryPeriodicPublishEnabled() const;

	/** Returns true if we are in debug mode dry mode and shouldn't extract incoming buffer to local project */
	bool ShouldShowImportWizard() const;

private:
	/**
	 * Whether to use a custom listen hostname address for the TCP server.
	 *
	 * By Default, Storm Sync will use the same value as defined in UDPMessagingSettings > Unicast Endpoint
	 * (or `-UDPMESSAGING_TRANSPORT_UNICAST=` command line flag)
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server")
	bool bOverrideServerAddress = false;
	
	/**
	 * The IP endpoint to listen to when starting off the tcp socket server listener.
	 *
	 * 0.0.0.0 will bind to the default network adapter on Windows, and all available network adapters on other operating systems.
	 *
	 * Note: Only used if bOverrideServerAddress is false
	 *
	 * Can be specified on the command line with `-StormSyncServerEndpoint=`
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server", meta=(EditCondition = "bOverrideServerAddress", EditConditionHides))
	FString TcpServerAddress;

	/**
	 * The port to listen on when starting off the tcp socket server listener.
	 * 
	 * Can be specified on the command line with `-StormSyncServerEndpoint=`
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server")
	uint16 TcpServerPort = 40990;

	/**
	 * Specifies the port offset range when trying to acquire an available socket for the Tcp server.
	 * The range of ports will be [TcpServerPort, TcpServerPort + TcpServerPortMaxOffset - 1].
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server")
	uint16 TcpServerPortMaxOffset = 10;
	
	/**
	 * User friendly name to describe this server on the network (defaults to hostname if left to empty)
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server")
	FString ServerName;

	/**
	 * Should this editor instance starts on startup a Storm Sync server endpoint (messaging and tcp)
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server")
	bool bAutoStartServer = true;

	/**
	 * The amount of time in seconds to consider a client to be inactive, if it has not sent data in this interval.
	 *
	 * Server listener will close the tcp connection for any inactive clients passed that delay.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server")
	uint32 InactiveTimeoutSeconds = 5;

	/**
	 * Debug Development config.
	 *
	 * When set to true, an ava pak buffer received over tcp connection will only be displayed in console and not extracted / installed to local project
	 */
	UPROPERTY(config, EditAnywhere, Category = "Server")
	bool bTcpDryRun = false;

	/**
	 * Delay time between attempts to re-establish outgoing connections that become disconnected or fail to connect
	 * 
	 * 0 disables reconnection
	 *
	 * Note that editor must be restarted for changes to this config to be taken into account.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Client")
	uint32 ConnectionRetryDelay = 0;

	/** The refresh frequency of the heartbeat event */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service Discovery", meta=(ClampMin = "0.1", ConfigRestartRequired=true, ForceUnits=s))
	float MessageBusHeartbeatPeriod = 1.f;

	/** How long we should wait before checking for message bus sources activity */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service Discovery", meta=(ClampMin = "0.1", ConfigRestartRequired=true, ForceUnits=s))
	float DiscoveryManagerTickInterval = 1.f;

	/** How long we should wait before a connected device becomes unresponsive */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service Discovery", meta=(ClampMin = "0.1", ConfigRestartRequired=true, ForceUnits=s))
	double MessageBusHeartbeatTimeout = 2.0;

	/** Subjects will be removed when their source has been unresponsive for this long */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service Discovery", meta=(ClampMin = "0.1", ConfigRestartRequired=true, ForceUnits=s))
	double MessageBusTimeBeforeRemovingInactiveSource = 15.0;

	/**
	 * Whether to allow Discovery Manager to broadcast a connect message at periodic interval (period time being the value of
	 * MessageBusTimeBeforeRemovingInactiveSource).
	 *
	 * This is a fail-safe mechanism to ensure editor instances can still discover each other in case both ends were inactive for
	 * an extended period of time.
	 */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Service Discovery", meta=(ConfigRestartRequired=true))
	bool bEnableDiscoveryPeriodicPublish = true;

	/** When set to true, an import wizard will be prompted to the user upon receiving a buffer request from network */
	UPROPERTY(config, EditAnywhere, Category = "UI")
	bool bShowImportWizard = false;

	/** Internal helper to return the value of UdpMessagingSettings UnicastEndpoint */
	bool GetUdpMessagingUnicastEndpoint(FString& OutUnicastHostname, FString& OutUnicastPort) const;
};
