// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StormSyncTransportTcpPackets.generated.h"

USTRUCT()
struct FStormSyncTransportTcpPacket
{
	GENERATED_BODY()

	/**
	 * An arbitrary "command" used when parsing JSON string to differentiate the incoming packets.
	 *
	 * Set in child structs.
	 */
	UPROPERTY()
	FString Command;

	/** Host name the packet was generated from */
	UPROPERTY()
	FString HostName;

	FStormSyncTransportTcpPacket()
		: HostName(FPlatformProcess::ComputerName())
	{
	}
};

/** This is an initial snapshot, sent upon connecting */
USTRUCT()
struct FStormSyncTransportTcpStatePacket : public FStormSyncTransportTcpPacket
{
	GENERATED_BODY()

	FStormSyncTransportTcpStatePacket()
		: FStormSyncTransportTcpPacket()
	{
		Command = TEXT("state");
	}
};

/** This is a socket "message" sent from receiver to notify sender of the amount of bytes received so far */
USTRUCT()
struct FStormSyncTransportTcpSizePacket : public FStormSyncTransportTcpPacket
{
	GENERATED_BODY()

	/** Number of bytes received so far */
	UPROPERTY()
	uint32 ReceivedBytes = 0;

	FStormSyncTransportTcpSizePacket() = default;
	
	explicit FStormSyncTransportTcpSizePacket(const uint32 InReceivedBytes)
		: FStormSyncTransportTcpPacket()
		, ReceivedBytes(InReceivedBytes)
	{
		Command = TEXT("size");
	}
};

/** This is a socket "message" sent from receiver to notify sender of the amount of bytes received so far */
USTRUCT()
struct FStormSyncTransportTcpTransferCompletePacket : public FStormSyncTransportTcpPacket
{
	GENERATED_BODY()

	FStormSyncTransportTcpTransferCompletePacket()
		: FStormSyncTransportTcpPacket()
	{
		Command = TEXT("transfer_complete");
	}
};