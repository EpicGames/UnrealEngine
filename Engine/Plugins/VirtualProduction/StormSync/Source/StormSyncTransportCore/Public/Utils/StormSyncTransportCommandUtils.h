// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

namespace UE::StormSync::Transport::Private
{
	static const TCHAR* ServerEndpointFlag = TEXT("StormSyncServerEndpoint=");
	static const TCHAR* NoServerAutoStartFlag = TEXT("NoStormSyncServerAutoStart");

	/**
	 * Returns value of command line -StormSyncServerEndpoint=hostname:1234 flag.
	 *
	 * Port value must be a valid digit. If "hostname" is omitted then the value of UStormSyncTransportSettings::TcpServerAddress
	 * is used from project settings with whatever Port value was given via command line (eg. 0.0.0.0:1234)
	 *
	 * OutEndpointValue will be updated only if the parsing was successful.
	 */
	STORMSYNCTRANSPORTCORE_API bool GetServerEndpointParam(FString& OutEndpointValue, const FString& InDefaultHostName = FString());
	
	/** Returns whether command line has either -StormSyncServerAutoStart or a valid -StormSyncServerEndpoint=<Port> */
	STORMSYNCTRANSPORTCORE_API bool IsServerAutoStartDisabled();
}
