// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StormSyncTransportCore : ModuleRules
	{
		public StormSyncTransportCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                }
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DeveloperSettings",
					"Engine",
					"MessagingCommon",
					"Networking",
					"Sockets",
					"StormSyncCore", 
				}
			);
        }
    }
}
