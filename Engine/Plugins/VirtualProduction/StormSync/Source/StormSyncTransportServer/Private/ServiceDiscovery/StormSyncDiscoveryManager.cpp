// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncDiscoveryManager.h"

#include "Async/TaskGraphInterfaces.h"
#include "HAL/RunnableThread.h"
#include "IStormSyncTransportServerModule.h"
#include "MessageEndpointBuilder.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncHeartbeatEmitter.h"
#include "StormSyncTransportMessages.h"
#include "StormSyncTransportNetworkUtils.h"
#include "StormSyncTransportServerLog.h"

FStormSyncDiscoveryManager::FStormSyncDiscoveryManager(double InHeartbeatTimeout, double InInactiveSourceTimeout, float InTickInterval, bool bInEnableDiscoveryPeriodicPublish)
	: bRunning(true)
	, Thread(nullptr)
	, DefaultHeartbeatTimeout(InHeartbeatTimeout)
	, DefaultDeadSourceTimeout(InInactiveSourceTimeout)
	, DefaultTickInterval(InTickInterval)
	, bEnableDiscoveryPeriodicPublish(bInEnableDiscoveryPeriodicPublish)
{
	MessageEndpoint = FMessageEndpoint::Builder(TEXT("StormSyncMessageHeartbeatManager (StormSyncDiscoveryManager)"))
		.ReceivingOnThread(ENamedThreads::GameThread)
		.Handling<FStormSyncTransportConnectMessage>(this, &FStormSyncDiscoveryManager::HandleConnectMessage)
		.Handling<FStormSyncTransportHeartbeatMessage>(this, &FStormSyncDiscoveryManager::HandleHeartbeatMessage)
		.Handling<FStormSyncTransportWakeupRequest>(this, &FStormSyncDiscoveryManager::HandleWakeupMessage);

	bRunning = MessageEndpoint.IsValid();
	if (bRunning)
	{
		UE_LOGF(LogStormSyncServer, Display, "FStormSyncDiscoveryManager::FStormSyncDiscoveryManager - Subscribe to messages");
		MessageEndpoint->Subscribe<FStormSyncTransportConnectMessage>();

		UE_LOGF(LogStormSyncServer, Display, "FStormSyncDiscoveryManager::FStormSyncDiscoveryManager - Start Thread");
		Thread = FRunnableThread::Create(this, TEXT("StormSyncDiscoveryManager"));
	}
}

FStormSyncDiscoveryManager::~FStormSyncDiscoveryManager()
{
	{
		FScopeLock Lock(&ConnectionLastActiveSection);

		// Disable the Endpoint message handling since the message could keep it alive a bit.
		if (MessageEndpoint)
		{
			MessageEndpoint->Disable();
			MessageEndpoint.Reset();
		}
	}

	bRunning = false;

	if (Thread)
	{
		Thread->Kill(true);
		Thread = nullptr;
	}
}

uint32 FStormSyncDiscoveryManager::Run()
{
	while (bRunning)
	{
		const double CurrentTime = FPlatformTime::Seconds();

		TArray<FStormSyncDelegateItem> DelegateQueue;
		{
			FScopeLock Lock(&ConnectionLastActiveSection);


			TArray<FMessageAddress> DisconnectedAddresses;
			for (TPair<FMessageAddress, FStormSyncConnectedMessageBusAddress>& Entry : ConnectedAddresses)
			{
				FMessageAddress MessageAddress = Entry.Key;
				FStormSyncConnectedMessageBusAddress& ConnectedAddress = Entry.Value;
				const FString MessageAddressUID = MessageAddress.ToString();

				const bool bPrevState = ConnectedAddress.bIsValid;

				const double ElapsedTime = CurrentTime - ConnectedAddress.LastActivityTime;
				const bool bIsValid = ElapsedTime < DefaultHeartbeatTimeout;

				if (bPrevState != bIsValid)
				{
					// Queue up a delegate event to be fired off later on
					DelegateQueue.Add(FStormSyncDelegateItem(
						FStormSyncDelegateItem::StateChange,
						MessageAddress,
						bIsValid ? EStormSyncConnectedDeviceState::State_Active : EStormSyncConnectedDeviceState::State_Unresponsive
					));
				}

				// Update state in stored connection
				ConnectedAddress.bIsValid = bIsValid;

				// Connection starting to be unresponsive, can be a real disconnect or an occasional lag spike
				if (!bIsValid)
				{
					// If we exceeded inactive time, consider this remote as disconnected
					if (ElapsedTime > DefaultDeadSourceTimeout)
					{
						DisconnectedAddresses.Add(MessageAddress);
					}
				}
			}

			// Handle inactive addresses
			for (FMessageAddress MessageAddress : DisconnectedAddresses)
			{
				UE_LOGF(LogStormSyncServer, Display, "FStormSyncDiscoveryManager::Run - %ls became invalid", *MessageAddress.ToString());

				// Stop Heartbeat for this recipient
				FStormSyncHeartbeatEmitter& HeartbeatEmitter = IStormSyncTransportServerModule::Get().GetHeartbeatEmitter();
				HeartbeatEmitter.StopHeartbeat(MessageAddress, MessageEndpoint);

				// Queue up a delegate event to notify about disconnection
				FStormSyncDelegateItem Item = FStormSyncDelegateItem(FStormSyncDelegateItem::Disconnection, MessageAddress, EStormSyncConnectedDeviceState::State_Disconnected);
				DelegateQueue.Add(Item);

				// Actually remove the address now
				ConnectedAddresses.Remove(MessageAddress);
			}
		}

		// Handle any delegate to fire off now
		BroadcastCoreDelegatesFromQueue(DelegateQueue);

		// Handle periodic connect message publish if enabled
		if (bEnableDiscoveryPeriodicPublish)
		{
			const double ElapsedTime = CurrentTime - LastPublishTime;
			if (ElapsedTime > DefaultDeadSourceTimeout)
			{
				PublishConnectMessage();
			}
		}

		FPlatformProcess::Sleep(FMath::Max<float>(0.1f, DefaultTickInterval));
	}
	return 0;
}

void FStormSyncDiscoveryManager::Stop()
{
	bRunning = false;
}

void FStormSyncDiscoveryManager::PublishConnectMessage()
{
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncServer, Error, "FStormSyncDiscoveryManager::PublishConnectMessage - Unable to send Connect Message cause Message Endpoint is invalid");
		return;
	}

	UE_LOGF(LogStormSyncServer, VeryVerbose, "FStormSyncTransportClientModule::PublishConnectMessage - Publish Connect Message ...");
	TUniquePtr<FStormSyncTransportConnectMessage> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportConnectMessage>());
	if (Message.IsValid())
	{
		Message->StormSyncServerAddressId = FStormSyncTransportNetworkUtils::GetServerEndpointMessageAddress();
		Message->StormSyncClientAddressId = FStormSyncTransportNetworkUtils::GetClientEndpointMessageAddress();

		LastPublishTime = FPlatformTime::Seconds();
		MessageEndpoint->Publish(Message.Release());
	}
}

void FStormSyncDiscoveryManager::SendWakeUp()
{
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncServer, Error, "FStormSyncDiscoveryManager::SendWakeUp - Unable to send Connect Message cause Message Endpoint is invalid");
		return;
	}

	UE_LOGF(LogStormSyncServer, VeryVerbose, "FStormSyncTransportClientModule::SendWakeUp - Sending Wakeup Request to all connections.");
	FStormSyncTransportWakeupRequest* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportWakeupRequest>();

	TArray<FMessageAddress> Connections;
	{
		FScopeLock ConnectionTimeLock(&ConnectionLastActiveSection);
		ConnectedAddresses.GetKeys(Connections);
	}
	
	MessageEndpoint->Send(Message, Connections);
}

TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> FStormSyncDiscoveryManager::GetMessageEndpoint() const
{
	return MessageEndpoint;
}

void FStormSyncDiscoveryManager::BroadcastCoreDelegatesFromQueue(const TArray<FStormSyncDelegateItem>& InDelegateQueue)
{
	for (const FStormSyncDelegateItem& DelegateItem : InDelegateQueue)
	{
		if (DelegateItem.DelegateType == FStormSyncDelegateItem::EDelegateType::StateChange)
		{
			// Notify editor this remote state changed (either responsive or unresponsive)
			FStormSyncCoreDelegates::OnServiceDiscoveryStateChange.Broadcast(DelegateItem.Address.ToString(), DelegateItem.State);
		}
		else if (DelegateItem.DelegateType == FStormSyncDelegateItem::EDelegateType::Disconnection)
		{
			// Notify Editor this remote is considered disconnected
			FStormSyncCoreDelegates::OnServiceDiscoveryDisconnection.Broadcast(DelegateItem.Address.ToString());
		}
	}
}

void FStormSyncDiscoveryManager::HandleConnectMessage(const FStormSyncTransportConnectMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
{
	const FMessageAddress SenderMessageAddress = MessageContext->GetSender();
	UE_LOGF(LogStormSyncServer, VeryVerbose, "FStormSyncDiscoveryManager::HandleConnectMessage - Received connect Message from %ls: %ls", *SenderMessageAddress.ToString(), *InMessage.ToString());

	const FMessageAddress MessageEndpointAddress = MessageEndpoint->GetAddress();

	UE_LOGF(LogStormSyncServer, VeryVerbose, "FStormSyncDiscoveryManager::HandleConnectMessage - SenderMessageAddress: %ls, MessageEndpointAddress: %ls (Same: %ls)", *SenderMessageAddress.ToString(), *MessageEndpointAddress.ToString(), SenderMessageAddress == MessageEndpointAddress ? TEXT("true") : TEXT("false"));

	if (SenderMessageAddress != MessageEndpointAddress)
	{
		if (!IsConnectionRegistered(SenderMessageAddress))
		{
			// Add this message address to our list of connected message bus addresses
			RegisterConnection(SenderMessageAddress, !InMessage.StormSyncServerAddressId.IsEmpty());
		}

		bool bNotifyConnectionInfo = false;
		{
			FScopeLock ConnectionTimeLock(&ConnectionLastActiveSection);
			if (FStormSyncConnectedMessageBusAddress* FoundConnection = ConnectedAddresses.Find(SenderMessageAddress))
			{
				bNotifyConnectionInfo = !FoundConnection->bReceivedConnectionInfo;
				FoundConnection->bReceivedConnectionInfo = true;
			}
		}
		
		if (bNotifyConnectionInfo)
		{
			// Notify editor of incoming connection
			FStormSyncConnectedDevice ConnectedDevice;
			ConnectedDevice.State = EStormSyncConnectedDeviceState::State_Active;
			ConnectedDevice.MessageAddressId = SenderMessageAddress.ToString();
			ConnectedDevice.bIsServerRunning = !InMessage.StormSyncServerAddressId.IsEmpty();
			ConnectedDevice.StormSyncServerAddressId = InMessage.StormSyncServerAddressId;
			ConnectedDevice.StormSyncClientAddressId = InMessage.StormSyncClientAddressId;
			ConnectedDevice.HostName = InMessage.HostName;
			ConnectedDevice.ProjectName = InMessage.ProjectName;
			ConnectedDevice.ProjectDir = InMessage.ProjectDir;
			ConnectedDevice.InstanceType = InMessage.InstanceType;

			FStormSyncCoreDelegates::OnServiceDiscoveryConnection.Broadcast(SenderMessageAddress.ToString(), ConnectedDevice);
		}
	}
}

void FStormSyncDiscoveryManager::HandleWakeupMessage(const FStormSyncTransportWakeupRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
{
	FStormSyncCoreDelegates::OnServiceDiscoveryReceivedWakeup.Broadcast();
}

void FStormSyncDiscoveryManager::RegisterConnection(const FMessageAddress& InMessageAddress, bool bInIsServerRunning)
{
	{
		FScopeLock ConnectionTimeLock(&ConnectionLastActiveSection);
		check(ConnectedAddresses.Contains(InMessageAddress) == false);

		FStormSyncConnectedMessageBusAddress ConnectedAddress;
		ConnectedAddress.bIsValid = true;
		ConnectedAddress.bIsServerRunning = bInIsServerRunning;
		ConnectedAddress.LastActivityTime = FPlatformTime::Seconds();

		ConnectedAddresses.Add(InMessageAddress, ConnectedAddress);
	}

	// Start Heartbeat for this recipient
	FStormSyncHeartbeatEmitter& HeartbeatEmitter = IStormSyncTransportServerModule::Get().GetHeartbeatEmitter();
	HeartbeatEmitter.StartHeartbeat(InMessageAddress, MessageEndpoint);

	// Send back connect message so that this recipient knows about this editor instance
	UE_LOGF(LogStormSyncServer, Display, "FStormSyncDiscoveryManager::HandleConnectMessage - Send Connect Message to %ls...", *InMessageAddress.ToString());
	TUniquePtr<FStormSyncTransportConnectMessage> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportConnectMessage>());
	if (Message.IsValid())
	{
		Message->StormSyncServerAddressId = FStormSyncTransportNetworkUtils::GetServerEndpointMessageAddress();
		Message->StormSyncClientAddressId = FStormSyncTransportNetworkUtils::GetClientEndpointMessageAddress();
		MessageEndpoint->Send(Message.Release(), InMessageAddress);
	}
}

void FStormSyncDiscoveryManager::HandleHeartbeatMessage(const FStormSyncTransportHeartbeatMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
{
	const FMessageAddress MessageAddress = MessageContext->GetSender();

	UE_LOGF(LogStormSyncServer, VeryVerbose, "FStormSyncDiscoveryManager::HandleHeartbeatMessage - Received Heartbeat Message from %ls", *MessageAddress.ToString());
	
	// Handle revive connection in case we receive heartbeats again, most likely meaning
	// connection was marked here as inactive due to inactive timeout and cleaned up.
	//
	// This is likely caused by a debug session with breakpoints that are longer than the configured inactive timeout
	
	if (!IsConnectionRegistered(MessageAddress))
	{
		RegisterConnection(MessageAddress, InMessage.bIsServerRunning);
	}

	UpdateConnectionLastActive(MessageAddress);

	// Update Server status and check if it changed for this recipient since last heartbeat, if so notify editor
	if (UpdateServerStatus(MessageAddress, InMessage.bIsServerRunning))
	{
		// Notify editor this remote state changed (either running or stopped or unresponsive)
		FStormSyncCoreDelegates::OnServiceDiscoveryServerStatusChange.Broadcast(MessageAddress.ToString(), InMessage.bIsServerRunning);
	}
}

void FStormSyncDiscoveryManager::UpdateConnectionLastActive(const FMessageAddress& InAddress)
{
	FScopeLock ConnectionTimeLock(&ConnectionLastActiveSection);
	if (FStormSyncConnectedMessageBusAddress* ConnectedMessageBusAddress = ConnectedAddresses.Find(InAddress))
	{
		ConnectedMessageBusAddress->LastActivityTime = FPlatformTime::Seconds();
	}
}

bool FStormSyncDiscoveryManager::UpdateServerStatus(const FMessageAddress& InAddress, const bool bIsServerRunning)
{
	FScopeLock ConnectionTimeLock(&ConnectionLastActiveSection);
	if (FStormSyncConnectedMessageBusAddress* ConnectedMessageBusAddress = ConnectedAddresses.Find(InAddress))
	{
		// Server status changed for this recipient
		if (ConnectedMessageBusAddress->bIsServerRunning != bIsServerRunning)
		{
			ConnectedMessageBusAddress->bIsServerRunning = bIsServerRunning;
			return true;
		}
	}
	return false;
}
