// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "MessageEndpoint.h"
#include "StormSyncTransportMessages.h"

enum class EStormSyncConnectedDeviceState : uint8;

/** Data holder for an active message bus connection over storm sync bus */
struct FStormSyncConnectedMessageBusAddress
{
	/**
	 * Updated each time connection becomes active or inactive and reflects whether elapsed time since LastActivityTime
	 * exceeded the heartbeat timeout configured in Storm Sync Transport Settings.
	 */
	bool bIsValid = false;

	/** Represents the time of last activity. Updated on connection and on each incoming heartbeat message */
	double LastActivityTime = 0.0;

	/** Represents remote server local endpoint status. Whether it is currently running and able to receive incoming connections */
	bool bIsServerRunning = false;

	/** Indicate if a connection info has been received for this connection. */
	bool bReceivedConnectionInfo = false;

	/** Default constructor */
	FStormSyncConnectedMessageBusAddress() = default;
};

/**
 * Data holder for delegate message queue.
 *
 * Holds the type of delegate to broadcast, message address and state.
 *
 * Introduced to prevent firing off delegate within a critical section in DiscoverManager Run(). Instead,
 * it's now building up a queue of delegates to fire off when stepping out critical section path.
 */
struct FStormSyncDelegateItem
{
	/** Simple enum to list possible types of delegate handled by the queue */
	enum EDelegateType
	{
		StateChange,
		Disconnection
	};

	/** Which type of event to trigger */
	EDelegateType DelegateType;

	/** Message address of the remote connection */
	FMessageAddress Address;

	/**
	 * Hold the state of the connection to broadcast with StateChanges (either responsive or unresponsive)
	 *
	 * Only relevant with StateChange type.
	 */
	EStormSyncConnectedDeviceState State;

	/** Default constructor */
	FStormSyncDelegateItem(const EDelegateType InDelegateType, const FMessageAddress& InAddress, const EStormSyncConnectedDeviceState InState)
		: DelegateType(InDelegateType)
		, Address(InAddress)
		, State(InState)
	{
	}
};

/** A class to asynchronously discover message bus sources. */
class FStormSyncDiscoveryManager : public FRunnable
{
public:
	FStormSyncDiscoveryManager(double InHeartbeatTimeout, double InInactiveSourceTimeout, float InTickInterval, bool bInEnableDiscoveryPeriodicPublish);
	~FStormSyncDiscoveryManager();

	//~ Begin FRunnable interface
	virtual uint32 Run() override;
	virtual void Stop() override;
	//~ End FRunnable interface

	/** Broadcasts / Publish a FStormSyncTransportConnectMessage over the network */
	void PublishConnectMessage();

	void SendWakeUp();
	
	/** Returns the underlying endpoint so that outside code could send message directly to it (w/o relying on publish) */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const;

private:
	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Thread safe bool for stopping the thread */
	std::atomic<bool> bRunning;

	/** For the thread */
	FRunnableThread* Thread;

	/**
	 * Map of currently connected message bus recipients.
	 * 
	 * Key is the message address, value is a FStormSyncConnectedMessageBusAddress with info like last active time and status.
	 */
	TMap<FMessageAddress, FStormSyncConnectedMessageBusAddress> ConnectedAddresses;

	/** Critical section to allow for ThreadSafe updating of the connection time */
	FCriticalSection ConnectionLastActiveSection;

	/** Cached setting for MessageBusHeartbeatTimeout */
	double DefaultHeartbeatTimeout;

	/** Cached setting for MessageBusTimeBeforeRemovingInactiveSource */
	double DefaultDeadSourceTimeout;

	/** Cached setting for DiscoveryManagerTickInterval */
	float DefaultTickInterval;
	
	/** Cached setting for bEnableDiscoveryPeriodicPublish */
	bool bEnableDiscoveryPeriodicPublish;

	/** Represents the time of last connect message publish. Only relevant if periodic publish is enabled (bEnableDiscoveryPeriodicPublish true) */
	double LastPublishTime = 0.0;

	/**
	 * Goes through the delegate queue and broadcast appropriate core delegate to notify
	 * outside code of any changes in service discovery.
	 */
	static void BroadcastCoreDelegatesFromQueue(const TArray<FStormSyncDelegateItem>& InDelegateQueue);

	/** Callback handler to receive FStormSyncTransportConnectMessage messages */
	void HandleConnectMessage(const FStormSyncTransportConnectMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext);

	void HandleWakeupMessage(const FStormSyncTransportWakeupRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext);
	
	bool IsConnectionRegistered(const FMessageAddress& InMessageAddress)
	{
		FScopeLock ConnectionTimeLock(&ConnectionLastActiveSection);
		return ConnectedAddresses.Contains(InMessageAddress);
	}
	
	/**
	 * Handles an incoming connect message by registering a new connection internally (adding to ConnectedAddresses), starting heartbeats for this recipient
	 * and sending back a connect message to original sender.
	 *
	 * You must ensure connection is not already registered before calling this method.
	 */
	void RegisterConnection(const FMessageAddress& InMessageAddress, bool bInIsServerRunning);

	/** Callback handler to receive FStormSyncTransportHeartbeatMessage messages */
	void HandleHeartbeatMessage(const FStormSyncTransportHeartbeatMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext);

	/** ThreadSafe update of the last active time */
	void UpdateConnectionLastActive(const FMessageAddress& InAddress);

	/**
	 * ThreadSafe update of the server status for remote address.
	 *
	 * Returns whether server running state changed since last heartbeat, and whether server status change event should be broadcast. 
	 */
	bool UpdateServerStatus(const FMessageAddress& InAddress, const bool bIsServerRunning);
};
