// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncHeartbeatEmitter.h"

#include "HAL/RunnableThread.h"
#include "IStormSyncTransportServerModule.h"
#include "StormSyncTransportMessages.h"
#include "StormSyncTransportServerLog.h"
#include "StormSyncTransportSettings.h"

bool FStormSyncHeartbeatEmitter::FHeartbeatRecipient::operator==(const FHeartbeatRecipient& Other) const
{
	return Other.MessageEndpoint == MessageEndpoint && Other.ConnectionAddress == ConnectionAddress;
}

FStormSyncHeartbeatEmitter::FStormSyncHeartbeatEmitter()
	: bIsRunning(false)
	, HeartbeatFrequencyInMs(FMath::Max(0.1f, GetDefault<UStormSyncTransportSettings>()->GetMessageBusHeartbeatPeriod()) * 1000.0f)
	, HeartbeatEvent(nullptr)
{
}

void FStormSyncHeartbeatEmitter::StartHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint)
{
	UE_LOGF(LogStormSyncServer, Verbose, "Start Heartbeat for %ls", *RecipientAddress.ToString());

	if (!bIsRunning)
	{
		bIsRunning = true;
		HeartbeatEvent = FGenericPlatformProcess::GetSynchEventFromPool();
		check(HeartbeatEvent);
		Thread.Reset(FRunnableThread::Create(this, TEXT("StormSyncHeartbeatEmitter")));
	}

	FScopeLock Lock(&CriticalSection);
	HeartbeatRecipients.Add({ MessageEndpoint, RecipientAddress });
}

void FStormSyncHeartbeatEmitter::StopHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint)
{
	FScopeLock Lock(&CriticalSection);
	HeartbeatRecipients.Remove({ MessageEndpoint, RecipientAddress });
}

uint32 FStormSyncHeartbeatEmitter::Run()
{
	while (bIsRunning)
	{
		check(HeartbeatEvent);
		HeartbeatEvent->Wait(HeartbeatFrequencyInMs);

		FScopeLock Lock(&CriticalSection);
		
		// Make sure we were not told to exit during the wait
		if (bIsRunning) 
		{
			for (const FHeartbeatRecipient& Recipient : HeartbeatRecipients)
			{
				if (const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = Recipient.MessageEndpoint.Pin())
				{
					UE_LOGF(LogStormSyncServer, VeryVerbose, "Sending Heartbeat Message to %ls", *Recipient.ConnectionAddress.ToString());
					if (FStormSyncTransportHeartbeatMessage* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportHeartbeatMessage>())
					{
						Message->bIsServerRunning = false;
						if (const IStormSyncTransportServerModule* ServerModule = FModuleManager::GetModulePtr<IStormSyncTransportServerModule>("StormSyncTransportServer"))
						{
							Message->bIsServerRunning = ServerModule->IsRunning();
						}
						
						MessageEndpoint->Send(Message, Recipient.ConnectionAddress);
					}
				}
			}
		}
	}

	return 0;
}

void FStormSyncHeartbeatEmitter::Exit()
{
	if (!bIsRunning)
	{
		return;
	}

	bIsRunning = false;
	{
		FScopeLock Lock(&CriticalSection);
		HeartbeatRecipients.Empty();
	}

	if (HeartbeatEvent)
	{
		HeartbeatEvent->Trigger();
	}

	Thread->WaitForCompletion();

	if (HeartbeatEvent)
	{
		FGenericPlatformProcess::ReturnSynchEventToPool(HeartbeatEvent);
		HeartbeatEvent = nullptr;
	}
}
