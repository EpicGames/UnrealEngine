// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"

/**
 * Heavily based on FLiveLinkHeartbeatEmitter
 * 
 * Sends out heartbeat signals at the rate defined by StormSyncTransportSettings::MessageBusHeartbeatFrequency to all recipients.
 *
 * Recipients for the heartbeat can be added by calling StartHeartbeat().
 * 
 * On the first call to StartHeartbeat a thread is spawned that keeps sending the heartbeat to everyone.
 * The heartbeats and the sender thread can be stopped by calling Exit().
 */
class FStormSyncHeartbeatEmitter : public FRunnable
{
public:
	FStormSyncHeartbeatEmitter();

	/**
	 * Start sending a heartbeat to the specified Recipient by using the given MessageEndpoint.
	 * @note This will spawn a new thread when first called.
	 * @note It is assumed that each source that sends out heartbeats has its own MessageEndpoint.
	 */
	void StartHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint);

	/**
	 * Stop sending a heartbeat to the specified Recipient.
	 */
	void StopHeartbeat(const FMessageAddress& RecipientAddress, const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe>& MessageEndpoint);

	//~ Begin FRunnable interface
	virtual uint32 Run() override;
	virtual void Exit() override;
	//~ End FRunnable interface

private:
	/** Data holder for a remote heartbeat recipient */
	struct FHeartbeatRecipient
	{
		/** Weak pointer to the message endpoint for this recipient. Used to send heartbeats at a regular interval to the connection address */
		TWeakPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

		/** Message address for the remote recipient */
		FMessageAddress ConnectionAddress;

		bool operator==(const FHeartbeatRecipient& Other) const;
	};

	/** Critical section for heartbeat recipients access */
	mutable FCriticalSection CriticalSection;

	/** List of registered recipients for heartbeat messages. Added and removed respectively with StartHeartbeat / StopHeartbeat */
	TArray<FHeartbeatRecipient> HeartbeatRecipients;

	/** Whether this runnable should continue running */
	std::atomic<bool> bIsRunning;

	/** Cached heartbeat frequency (tied to Storm Sync Transport Settings MessageBusHeartbeatFrequency) to send heartbeats at a fixed interval */
	float HeartbeatFrequencyInMs;

	/** Waitable event for heartbeats */
	FEvent* HeartbeatEvent;

	/** Unique pointer to our thread, first started on the first StartHeartbeat */
	TUniquePtr<FRunnableThread> Thread;
};
