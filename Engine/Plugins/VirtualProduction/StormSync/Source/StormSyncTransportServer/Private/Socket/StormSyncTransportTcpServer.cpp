// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportTcpServer.h"

#include "Common/TcpListener.h"
#include "StormSyncTransportServerLog.h"
#include "StormSyncTransportSettings.h"
#include "StormSyncTransportTcpPackets.h"

FStormSyncTransportTcpServer::FStormSyncTransportTcpServer(const FIPv4Address& InEndpointAddress, const uint16 InEndpointPort, const uint32 InInactiveTimeoutSeconds)
	: DefaultInactiveTimeoutSeconds(InInactiveTimeoutSeconds)
	, DesiredEndpoint(InEndpointAddress, InEndpointPort)
{
	Thread = FRunnableThread::Create(this, TEXT("FStormSyncServerListener"), 128 * 1024, TPri_Normal);
}

FStormSyncTransportTcpServer::~FStormSyncTransportTcpServer()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	ReceivedBufferEvent.Clear();

	StopListening();
}

bool FStormSyncTransportTcpServer::Init()
{
	return true;
}

void FStormSyncTransportTcpServer::Stop()
{
	bStopping = true;
}

uint32 FStormSyncTransportTcpServer::Run()
{
	double LastTime = FPlatformTime::Seconds();
	constexpr float IdealFrameTime = 1.0f / 30.0f;

	bool bListenerIsRunning = true;

	while (bListenerIsRunning && !bStopping)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		const double DeltaTime = CurrentTime - LastTime;
		LastTime = CurrentTime;

		if (DeltaTime > 0.1)
		{
			UE_LOGF(LogStormSyncServer, Warning, "Hitch detected; %.3f seconds since prior tick", DeltaTime);
		}

		bListenerIsRunning = Tick();

		// Throttle main thread main fps by sleeping if we still have time
		FPlatformProcess::Sleep(FMath::Max<float>(0.01f, IdealFrameTime - (FPlatformTime::Seconds() - CurrentTime)));
	}

	return 0;
}

bool FStormSyncTransportTcpServer::Tick()
{
	// Dequeue pending connections
	{
		TPair<FIPv4Endpoint, TSharedPtr<FSocket>> Connection;
		while (PendingConnections.Dequeue(Connection))
		{
			FIPv4Endpoint ClientEndpoint = Connection.Key;
			
			TUniquePtr<FClientConnection> ClientConnection = MakeUnique<FClientConnection>(Connection.Value);
			ClientConnection->InactiveTimeout = DefaultInactiveTimeoutSeconds;
			ClientConnection->LastActivityTime = FPlatformTime::Seconds();

			ClientConnections.Add(ClientEndpoint, MoveTemp(ClientConnection));

			// Send current state upon connection
			FStormSyncTransportTcpStatePacket StatePacket;
			SendMessage(CreateMessage(StatePacket), ClientEndpoint);
		}
	}
	
	// Parse incoming data from remote connections
	for (const TPair<FIPv4Endpoint, TUniquePtr<FClientConnection>>& Connection : ClientConnections)
	{
		const FIPv4Endpoint& ClientEndpoint = Connection.Key;

		if (!Connection.Value)
		{
			// Failsafe early return if connection invalid
			continue;
		}

		TSharedPtr<FSocket> ClientSocket = Connection.Value->Socket;

		uint32 PendingDataSize = 0;
		while (ClientSocket->HasPendingData(PendingDataSize))
		{
			TArray<uint8> Buffer;
			Buffer.AddUninitialized(PendingDataSize);
			int32 BytesRead = 0;
			if (!ClientSocket->Recv(Buffer.GetData(), PendingDataSize, BytesRead, ESocketReceiveFlags::None))
			{
				UE_LOGF(LogStormSyncServer, Error, "Error while receiving data via endpoint %ls", *ClientEndpoint.ToString());
				continue;
			}

			Connection.Value->LastActivityTime = FPlatformTime::Seconds();
			TArray<uint8>& MessageBuffer = Connection.Value->ReceiveBuffer;

			bool bHasReceivedSize = Connection.Value->BufferExpectedSize != 0;
			
			for (int32 i = 0; i < BytesRead; ++i)
			{
				MessageBuffer.Add(Buffer[i]);
			
				// First 4 expected bytes represents the Buffer size we're gonna get
				if (!bHasReceivedSize && i == 3)
				{
					const uint32 Size = *reinterpret_cast<const uint32*>(MessageBuffer.GetData());
			
					UE_LOGF(LogStormSyncServer, Verbose, "Received size: %d. Setting it for %ls", Size, *ClientEndpoint.ToString());
					Connection.Value->BufferExpectedSize = Size;
					Connection.Value->BufferStartTime = FPlatformTime::Seconds();
					bHasReceivedSize = true;
					MessageBuffer.Empty();
				}
			}
			
			if (bHasReceivedSize)
			{
				HandleIncomingBuffer(ClientEndpoint, ClientSocket, MessageBuffer);
			}
		}
	}

	CleanUpDisconnectedSockets();

	if (IsEngineExitRequested())
	{
		return false;
	}

	return true;
}

bool FStormSyncTransportTcpServer::StartListening()
{
	check(!SocketListener);
	
	const uint16 MaxPortOffset = UStormSyncTransportSettings::Get().GetTcpServerPortMaxOffset();
	
	FSocket* RawSocket = nullptr;

	for (uint16 PortOffset = 0; PortOffset < MaxPortOffset && !RawSocket; ++PortOffset)
	{
		FIPv4Endpoint ListenEndPoint(DesiredEndpoint.Address, DesiredEndpoint.Port + PortOffset);

		RawSocket = FTcpSocketBuilder(TEXT("FStormSyncTransportTcpServer"))
		.BoundToEndpoint(ListenEndPoint)
		.Listening(8) // Max number of connections in queue before refusing them
		.WithSendBufferSize(2 * 1024 * 1024).Build();
	}

	if (RawSocket)
	{
		// Wrap socket in a smart pointer so it is managed. It is not managed by FTcpListener.
		Socket.Reset(RawSocket);

		const FTimespan SleepTime =  FTimespan::FromSeconds(1);	// default argument.
		constexpr bool bSocketReusable = false;

		SocketListener = MakeUnique<FTcpListener>(*Socket, SleepTime, bSocketReusable);

		if (SocketListener->IsActive())
		{
			SocketListener->OnConnectionAccepted().BindRaw(this, &FStormSyncTransportTcpServer::OnIncomingConnection);
			UE_LOGF(LogStormSyncServer, Display, "Started listening on %ls", *SocketListener->GetLocalEndpoint().ToString());
			return true;
		}
	}

	Socket.Reset();
	UE_LOGF(LogStormSyncServer, Error, "Could not create Tcp Listener!");
	return false;
}

void FStormSyncTransportTcpServer::StopListening()
{
	bStopping = true;

	if (SocketListener.IsValid())
	{
		UE_LOGF(LogStormSyncServer, Display, "No longer listening on %ls", *SocketListener->GetLocalEndpoint().ToString());
		SocketListener.Reset();
	}

	Socket.Reset();

	for (const TTuple<FIPv4Endpoint, TUniquePtr<FClientConnection>>& Connection : ClientConnections) 
	{
		if (Connection.Value && Connection.Value->Socket.IsValid())
		{
			Connection.Value->Socket->Close();
		}
	}

	ClientConnections.Empty();
	PendingConnections.Empty();
	PendingConnectionsToDisconnect.Empty();
}

FString FStormSyncTransportTcpServer::GetEndpointAddress() const
{
	if (SocketListener && SocketListener->IsActive())
	{
		return SocketListener->GetLocalEndpoint().ToString();
	}

	return FString(); 
}

bool FStormSyncTransportTcpServer::IsActive() const
{
	return SocketListener.IsValid() && SocketListener->IsActive();
}

bool FStormSyncTransportTcpServer::OnIncomingConnection(FSocket* InSocket, const FIPv4Endpoint& InEndpoint)
{
	UE_LOGF(LogStormSyncServer, Display, "Incoming connection via %ls:%d", *InEndpoint.Address.ToString(), InEndpoint.Port);

	InSocket->SetNoDelay(true);
	PendingConnections.Enqueue(TPair<FIPv4Endpoint, TSharedPtr<FSocket>>(InEndpoint, MakeShareable(InSocket)));

	return true;
}

void FStormSyncTransportTcpServer::HandleIncomingBuffer(const FIPv4Endpoint& InEndpoint, const TSharedPtr<FSocket>& InClientSocket, const TArray<uint8>& InBytes)
{
	const int32 ReceivedBufferSize = InBytes.Num();
	UE_LOGF(LogStormSyncServer, Verbose, "\tHandle Incoming buffer via %ls:%d (Total Buffer Size: %d)", *InEndpoint.Address.ToString(), InEndpoint.Port, ReceivedBufferSize);

	// Send back to client the size buffer we received so far
	SendMessage(CreateMessage(FStormSyncTransportTcpSizePacket(ReceivedBufferSize)), InEndpoint);

	if (ClientConnections.Contains(InEndpoint))
	{
		const uint32 BufferExpectedSize = ClientConnections.FindChecked(InEndpoint)->BufferExpectedSize;
		if (ReceivedBufferSize == BufferExpectedSize)
		{
			UE_LOGF(LogStormSyncServer, Display, "\tWe received full buffer!!");
			
			// Send back to client transfer complete packet, so that it knows we're done with the transfer
			SendMessage(CreateMessage(FStormSyncTransportTcpTransferCompletePacket()), InEndpoint);
			
			// Trigger delegates, queue up connection cleanup
			HandleReceivedBuffer(InEndpoint, InClientSocket, InBytes);

		}
	}
}

void FStormSyncTransportTcpServer::HandleReceivedBuffer(const FIPv4Endpoint& InEndpoint, const TSharedPtr<FSocket>& InClientSocket, const TArray<uint8>& InBytes)
{
	UE_LOGF(LogStormSyncServer, Display, "Handle received buffer via %ls (Buffer Size: %d)", *InEndpoint.ToString(), InBytes.Num());
	ReceivedBufferEvent.Broadcast(InEndpoint, InClientSocket, MakeShared<FStormSyncBuffer>(InBytes));

	// Fully received buffer, clean up connection and the state associated with it
	PendingConnectionsToDisconnect.Enqueue(InEndpoint);
}

void FStormSyncTransportTcpServer::CleanUpDisconnectedSockets()
{
	TArray<FIPv4Endpoint> DisconnectedClients;

	// Check for clients inactivity
	const double CurrentTime = FPlatformTime::Seconds();
	for (const TTuple<FIPv4Endpoint, TUniquePtr<FClientConnection>>& Connection : ClientConnections)
	{
		const FIPv4Endpoint& RemoteEndpoint = Connection.Key;
		const float ClientTimeout = Connection.Value->InactiveTimeout;
		if (CurrentTime - Connection.Value->LastActivityTime > ClientTimeout)
		{
			UE_LOGF(LogStormSyncServer, Warning, "Client %ls has been inactive for more than %.1fs -- closing connection", *RemoteEndpoint.ToString(), ClientTimeout);
			DisconnectedClients.Add(RemoteEndpoint);
		}
	}

	// Dequeue pending connections to remove
	{
		FIPv4Endpoint ClientEndpoint;
		while (PendingConnectionsToDisconnect.Dequeue(ClientEndpoint))
		{
			DisconnectedClients.AddUnique(ClientEndpoint);
		}
	}

	for (const FIPv4Endpoint& DisconnectedClient : DisconnectedClients)
	{
		DisconnectClient(DisconnectedClient);
	}
}

void FStormSyncTransportTcpServer::DisconnectClient(const FIPv4Endpoint& InClientEndpoint)
{
	UE_LOGF(LogStormSyncServer, Display, "Client %ls disconnected", *InClientEndpoint.ToString());

	ClientConnections.Remove(InClientEndpoint);
}

bool FStormSyncTransportTcpServer::SendMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint)
{
	if (ClientConnections.Contains(InEndpoint))
	{
		const TSharedPtr<FSocket> ClientSocket = ClientConnections.FindChecked(InEndpoint)->Socket;
		if (!ClientSocket.IsValid())
		{
			return false;
		}

		UE_LOGF(LogStormSyncServer, Verbose, "Sending message to %ls - Message: %ls", *InEndpoint.ToString(), *InMessage);
		int32 BytesSent = 0;
		return ClientSocket->Send(reinterpret_cast<uint8*>(TCHAR_TO_UTF8(*InMessage)), InMessage.Len() + 1, BytesSent);
	}

	UE_LOGF(LogStormSyncServer, Verbose, "Trying to send message to disconnected client %ls - Message: %ls", *InEndpoint.ToString(), *InMessage);
	return false;
}

void FStormSyncTransportTcpServer::FSocketDeleter::operator()(FSocket* InSocket) const
{
	if (InSocket)
	{
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(InSocket);
	}
}