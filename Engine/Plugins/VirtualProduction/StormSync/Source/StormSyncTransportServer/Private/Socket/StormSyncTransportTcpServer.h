// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "HAL/Runnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "JsonObjectConverter.h"
#include "StormSyncCommonTypes.h"

class FTcpListener;

/** Holds connection data for a client endpoint */
struct FClientConnection
{
	/** Currently connected and active client socket */
	TSharedPtr<FSocket> Socket;

	/** Default inactive timeout in seconds for a connect client */
	uint32 InactiveTimeout;

	/** Platform time of last activity for a connected client (updated when receiving data) */
	double LastActivityTime;

	/** Expected buffer size for a connected client. Used to determine when a buffer has been fully received */
	uint32 BufferExpectedSize;

	/** Raw buffer received so far for a connected client. */
	TArray<uint8> ReceiveBuffer;

	/**
	 * Start time for a connected client.
	 *
	 * Not used right now, but could be used to display the amount of time it took to transfer
	 * the buffer and / or send back this info to clients.
	 */
	double BufferStartTime;

	/** Default constructor. A connection must have an associated socket. */
	explicit FClientConnection(const TSharedPtr<FSocket>& Socket)
		: Socket(Socket)
		, InactiveTimeout(0)
		, LastActivityTime(0)
		, BufferExpectedSize(0)
		, BufferStartTime(0)
	{
	}
};

/**
 * Implements a TCP listener able to receive incoming connections.
 *
 * Heavily based on FSwitchboardListener implementation.
 *
 * Mainly used to receive a raw buffer serialized from StormSyncCoreUtils methods to transfer a content spak.
 */
class FStormSyncTransportTcpServer : public FRunnable
{
public:
	explicit FStormSyncTransportTcpServer(const FIPv4Address& InEndpointAddress, const uint16 InEndpointPort, const uint32 InInactiveTimeoutSeconds);
	virtual ~FStormSyncTransportTcpServer() override;

	//~ Begin FRunnable interface
	virtual bool Init() override;
	virtual void Stop() override;
	virtual uint32 Run() override;
	//~ End FRunnable interface

	/**
	 * Event delegate that is executed when a buffer is fully received and ready to be extracted
	 *
	 * @param Endpoint The first parameter is the client endpoint.
	 * @param Socket The second parameter is the client socket.
	 * @param Buffer The third parameter is a shared pointer to fully received buffer.
	 */
	DECLARE_EVENT_ThreeParams(FStormSyncServerListener, FOnReceivedBufferEvent, const FIPv4Endpoint&, const TSharedPtr<FSocket>&, const FStormSyncBufferPtr&);
	
	/** Event a buffer is fully received and ready to be extracted */
	FOnReceivedBufferEvent& OnReceivedBuffer()
	{
		return ReceivedBufferEvent;
	}

	/**
	 * Main tick handler and throttled by Run(). Dequeues pending connections, parse incoming data and cleans up
	 * disconnected clients.
	 *
	 * Returns false if RequestEngineExit was called.
	 */
	bool Tick();

	/**
	 * Creates a new FTcpListener from configured Endpoint.
	 *
	 * Must be called (once) from outside code.
	 *
	 * @return true if we were able to create the listener
	 */
	bool StartListening();

	/**
	 * Stops the runnable (Run() will go through on next tick), dispose of created socket listener and close any active connect clients.
	 */
	void StopListening();

	/** Returns endpoint address we're currently listening on (ip:port) or empty string if not active. */
	FString GetEndpointAddress() const;

	/** Returns whether underlying FTcpListener has been created, is currently active and listening for incoming connections */
	bool IsActive() const;

private:
	/** The amount of time in seconds to consider a client to be inactive, if it has not sent data in this interval. */
	uint32 DefaultInactiveTimeoutSeconds;

	/** The desired endpoint for creating the tcp listener. The actual endpoint may be different in case of collision. */
	FIPv4Endpoint DesiredEndpoint;
	
	struct FSocketDeleter
	{
		void operator()(FSocket* InSocket) const;
	};
	TUniquePtr<FSocket, FSocketDeleter> Socket;

	/** Our instance of tcp listener, created in StartListening() */
	TUniquePtr<FTcpListener> SocketListener;

	/** Queue of remote pending connections added when tcp listener is accepting a new connection */
	TQueue<TPair<FIPv4Endpoint, TSharedPtr<FSocket>>, EQueueMode::Spsc> PendingConnections;
	
	/** Queue of remote endpoints to dispose and disconnect */
	TQueue<FIPv4Endpoint, EQueueMode::Spsc> PendingConnectionsToDisconnect;

	/** Map of currently connected and active clients */
	TMap<FIPv4Endpoint, TUniquePtr<FClientConnection>> ClientConnections;

	/** For the thread */
	std::atomic<bool> bStopping = false;

	/** Holds the thread object. */
	FRunnableThread* Thread;
	
	/** Multicast delegate that will broadcast a notification when a buffer is fully received and ready to be extracted */
	FOnReceivedBufferEvent ReceivedBufferEvent;

	/** Handler for tcp listener OnConnectionAccepted. Queues the incoming socket / endpoint to pending connections */
	bool OnIncomingConnection(FSocket* InSocket, const FIPv4Endpoint& InEndpoint);
	
	/**
	 * Called from Tick and handles an incoming chunk of data for a given connection.
	 *
	 * We respond back to clients the overall number of bytes we received so far, and checks against expected size (parsed from tcp stream "header")
	 * if we fully received the buffer in which case we trigger the ReceivedBufferEvent delegate.
	 */
	void HandleIncomingBuffer(const FIPv4Endpoint& InEndpoint, const TSharedPtr<FSocket>& InClientSocket, const TArray<uint8>& InBytes);

	/** Called from HandleIncomingBuffer() and simply broadcast ReceivedBufferEvent delegate with endpoint, connected socket and received buffer. */
	void HandleReceivedBuffer(const FIPv4Endpoint& InEndpoint, const TSharedPtr<FSocket>& InClientSocket, const TArray<uint8>& InBytes);
	
	/**
	 * Called from Tick() and checks if any connected clients has been inactive more than our DefaultInactiveTimeoutSeconds (5 seconds).
	 *
	 * If it's inactive, clears up the connect by calling DisconnectClient()
	 */
	void CleanUpDisconnectedSockets();

	/** Clears up the connected client by disposing every map related to that client endpoint */
	void DisconnectClient(const FIPv4Endpoint& InClientEndpoint);

	/** Template helper to serialize a given struct into JSON for sending over tcp socket */
	template <typename InStructType>
	FString CreateMessage(const InStructType& InStruct)
	{
		FString Message;
		const bool bMessageOk = FJsonObjectConverter::UStructToJsonObjectString(InStruct, Message);
		check(bMessageOk);
		return Message;
	}

	/**
	 * To use in conjunction with CreateMessage()
	 *
	 * Takes a raw string and send it on provided endpoint, if we have a current active connection for it
	 * (in ClientConnections map)
	 */
	bool SendMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint);
};
