// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportServerEndpoint.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "MessageEndpointBuilder.h"
#include "Socket/StormSyncTransportTcpServer.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncCoreUtils.h"
#include "StormSyncMemoryReader.h"
#include "StormSyncTransportNetworkUtils.h"
#include "StormSyncTransportServerLog.h"
#include "StormSyncTransportSettings.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "StormSyncTransportServerEndpoint"

FStormSyncTransportServerEndpoint::FStormSyncTransportServerEndpoint()
{
	MessageBusPtr = IMessagingModule::Get().GetDefaultBus();
	check(MessageBusPtr.IsValid());
}

FStormSyncTransportServerEndpoint::~FStormSyncTransportServerEndpoint()
{
	ShutdownMessaging();
	ShutdownTcpListener();
}

bool FStormSyncTransportServerEndpoint::StartTcpListener(const FIPv4Endpoint& InEndpoint)
{
	const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
	
	UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerEndpoint::StartTcpListener - Starting TCP listener with endpoint: %ls", *InEndpoint.ToString());

	TcpServer = MakeUnique<FStormSyncTransportTcpServer>(InEndpoint.Address, InEndpoint.Port, Settings->GetInactiveTimeoutSeconds());
	TcpServer->OnReceivedBuffer().AddRaw(this, &FStormSyncTransportServerEndpoint::HandleReceivedTcpBuffer);

	if (!TcpServer->StartListening())
	{
		FNumberFormattingOptions NumberFormattingOptions;
		// Ensure we display port as configured in settings, without formatting
		NumberFormattingOptions.UseGrouping = false;
		
		const FText ErrorMessage = FText::Format(
			LOCTEXT(
				"FailedStartStormSyncServer",
				"Storm Sync: Tcp server couldn't be started on port {0}\n\n"
				"If another editor with Storm Sync is running, please use a different port in Project Settings."
			),
			FText::AsNumber(InEndpoint.Port, &NumberFormattingOptions)
		);

#if WITH_EDITOR
		FNotificationInfo Info(ErrorMessage);
		Info.ExpireDuration = 5.0f;
		Info.bUseLargeFont = false;
		Info.HyperlinkText = LOCTEXT("Notification_OpenSettings", "Open Settings");
		Info.Hyperlink = FSimpleDelegate::CreateLambda([]()
		{
			UStormSyncTransportSettings::Get().OpenEditorSettingsWindow();
		});
		
		FSlateNotificationManager::Get().AddNotification(MoveTemp(Info));
#endif
		
		UE_LOGF(LogStormSyncServer, Error, "FStormSyncTransportServerEndpoint::StartTcpListener - %ls", *ErrorMessage.ToString());
		return false;
	}
	
	return true;
}

bool FStormSyncTransportServerEndpoint::StartTcpListener()
{
	
	const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
	check(Settings);

	const FString ServerEndpoint = Settings->GetServerEndpoint();
	
	FIPv4Endpoint Endpoint;
	if (FIPv4Endpoint::Parse(ServerEndpoint, Endpoint) || FIPv4Endpoint::FromHostAndPort(ServerEndpoint, Endpoint))
	{
		return StartTcpListener(Endpoint);
	}
	
	UE_LOGF(LogStormSyncServer, Error, "FStormSyncTransportServerEndpoint::StartTcpListener - Failed to parse endpoint '%ls'. TCP Socket listener will be disabled!", *ServerEndpoint);
	return false;
}

bool FStormSyncTransportServerEndpoint::IsRunning() const
{
	return MessageEndpoint.IsValid() && MessageEndpoint->IsEnabled();
}

bool FStormSyncTransportServerEndpoint::IsTcpServerActive() const
{
	return TcpServer.IsValid() && TcpServer->IsActive();
}

FString FStormSyncTransportServerEndpoint::GetTcpServerEndpointAddress() const
{
	FString Result;
	if (!IsTcpServerActive())
	{
		return Result;
	}

	Result = TcpServer->GetEndpointAddress();
	return Result;
}

TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> FStormSyncTransportServerEndpoint::GetMessageEndpoint() const
{
	return MessageEndpoint;
}

void FStormSyncTransportServerEndpoint::InitializeMessaging(const FString& InEndpointFriendlyName)
{
	// Message endpoint already created
	if (MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncServer, Warning, "Message endpoint already initialized");
		return;
	}

	const IMessageBusPtr MessageBus = MessageBusPtr.Pin();
	if (!MessageBus.IsValid())
	{
		UE_LOGF(LogStormSyncServer, Warning, "Default message bus is invalid");
		return;
	}

	const FString MessageEndpointName = FString::Printf(TEXT("StormSync%sEndpoint"), *InEndpointFriendlyName);
	UE_LOGF(LogStormSyncServer, Verbose, "FStormSyncTransportServerEndpoint::InitializeMessaging - Setting up %ls message endpoint", *MessageEndpointName);
	
	FMessageEndpointBuilder EndpointBuilder = FMessageEndpoint::Builder(*MessageEndpointName, MessageBus.ToSharedRef())
		.ReceivingOnThread(ENamedThreads::Type::GameThread)
		.Handling<FStormSyncTransportPingMessage>(this, &FStormSyncTransportServerEndpoint::HandlePingMessage)
		.Handling<FStormSyncTransportPongMessage>(this, &FStormSyncTransportServerEndpoint::HandlePongMessage)
		.Handling<FStormSyncTransportSyncRequest>(this, &FStormSyncTransportServerEndpoint::HandleSyncRequestMessage);
	
	// Build the endpoint
	MessageEndpoint = EndpointBuilder.Build();
	check(MessageEndpoint.IsValid());

	// Message subscribes
	UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerEndpoint::InitializeMessaging - Subscribe to messages");
	MessageEndpoint->Subscribe<FStormSyncTransportPingMessage>();
	MessageEndpoint->Subscribe<FStormSyncTransportSyncRequest>();
}

void FStormSyncTransportServerEndpoint::ShutdownMessaging()
{
	// Disable the Endpoint message handling since the message could keep it alive a bit.
	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Disable();
		MessageEndpoint.Reset();
	}
}

void FStormSyncTransportServerEndpoint::ShutdownTcpListener()
{
	if (TcpServer.IsValid())
	{
		TcpServer->OnReceivedBuffer().RemoveAll(this);
		TcpServer.Reset();
	}
}

void FStormSyncTransportServerEndpoint::HandlePingMessage(const FStormSyncTransportPingMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerEndpoint::HandlePingMessage - Received ping Message %ls", *InMessage.ToString());
	if (!MessageEndpoint.IsValid())
	{
		UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerEndpoint::HandlePingMessage - Invalid MessageEndpoint, can't pong back");
		return;
	}

	FStormSyncTransportPongMessage* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportPongMessage>();
	MessageEndpoint->Send(Message, InMessageContext->GetSender());
}

void FStormSyncTransportServerEndpoint::HandlePongMessage(const FStormSyncTransportPongMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerEndpoint::HandlePongMessage - Received pong Message %ls", *InMessage.ToString());
}

void FStormSyncTransportServerEndpoint::HandleReceivedTcpBuffer(const FIPv4Endpoint& InEndpoint, const TSharedPtr<FSocket>& InSocket, const FStormSyncBufferPtr& InBuffer)
{
	if (!InBuffer.IsValid())
	{
		UE_LOGF(LogStormSyncServer, Error, "Error received tcp buffer via %ls:%d (Buffer is invalid)", *InEndpoint.Address.ToString(), InEndpoint.Port);
		return;
	}

	UE_LOGF(LogStormSyncServer, Display, "Handle received tcp buffer via %ls:%d (Buffer Size: %lld)", *InEndpoint.Address.ToString(), InEndpoint.Port, InBuffer->Num());

	// Local copy of the endpoint we're receiving a buffer from
	FIPv4Endpoint Endpoint = InEndpoint;

	AsyncTask(ENamedThreads::GameThread, [InBuffer, Endpoint]()
	{
		// Dummy package descriptor with name set to imported filename (Note: Should consider serialize package descriptor along buffer header)
		FStormSyncPackageDescriptor PackageDescriptor;
		PackageDescriptor.Name = FText::Format(
			NSLOCTEXT("StormSyncTransportServerEndpoint", "ImportRequest", "Buffer Import request from {0}"),
			FText::FromString(Endpoint.Address.ToString())
		).ToString();

		const FStormSyncArchivePtr MemoryReader = MakeShared<FStormSyncMemoryReader>(InBuffer);
		FStormSyncCoreDelegates::OnRequestImportBuffer.Broadcast(PackageDescriptor, MemoryReader);
	});
}

void FStormSyncTransportServerEndpoint::HandleSyncRequestMessage(const FStormSyncTransportSyncRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext)
{
	UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerEndpoint::HandleSyncRequestMessage - Received sync request Message %ls (from %ls)", *InMessage.ToString(), *InMessageContext->GetSender().ToString());
	SendSyncResponse(InMessage, InMessageContext);
}

void FStormSyncTransportServerEndpoint::SendSyncResponse(const FStormSyncTransportSyncRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext) const
{
	FGuid MessageId = InMessage.MessageId;
	TArray<FName> LocalPackageNames = InMessage.PackageNames;
	FStormSyncPackageDescriptor LocalPackageDescriptor = InMessage.PackageDescriptor;
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> LocalMessageEndpoint = GetMessageEndpoint();
	FMessageAddress SenderAddress = InMessageContext->GetSender();

	Async(EAsyncExecution::Thread, [MessageId, MessageEndpointPtr = MoveTemp(LocalMessageEndpoint), PackageNames = MoveTemp(LocalPackageNames), RemotePackageDescriptor = MoveTemp(LocalPackageDescriptor), RemoteAddress = MoveTemp(SenderAddress)]()
	{
		// Build sync response message that is going to be sent over the network for a specific recipient
		TUniquePtr<FStormSyncTransportPushResponse> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportPushResponse>(MessageId));
		if (!Message.IsValid())
		{
			UE_LOGF(LogStormSyncServer, Error, "FStormSyncTransportServerEndpoint::SendSyncResponse - Sync response message is invalid");
			return;
		}

		// Perform diff here ...
		const TArray<FStormSyncFileModifierInfo> FileModifiers = FStormSyncCoreUtils::GetSyncFileModifiers(PackageNames, RemotePackageDescriptor.Dependencies);

		Message->PackageNames = PackageNames;
		Message->PackageDescriptor = RemotePackageDescriptor;
		Message->Modifiers = FileModifiers;

		Message->HostName = FStormSyncTransportNetworkUtils::GetServerName();
		Message->HostAddress = FStormSyncTransportNetworkUtils::GetCurrentTcpServerEndpointAddress();
		Message->HostAdapterAddresses = FStormSyncTransportNetworkUtils::GetLocalAdapterAddresses();

		UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerEndpoint::SendSyncResponse - Sending back to %ls sync response: %ls", *RemoteAddress.ToString(), *Message->ToString());
		if (FileModifiers.IsEmpty())
		{
			UE_LOGF(LogStormSyncServer, Display, "\tSending back empty list of modifiers, meaning pak are synced.");
		}

		MessageEndpointPtr->Send(Message.Release(), RemoteAddress);
	});
}

#undef LOCTEXT_NAMESPACE