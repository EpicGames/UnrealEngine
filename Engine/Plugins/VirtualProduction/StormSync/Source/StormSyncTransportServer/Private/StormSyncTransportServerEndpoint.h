// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IStormSyncTransportServerLocalEndpoint.h"
#include "StormSyncTransportMessages.h"

class FMessageEndpoint;
class FStormSyncTransportTcpServer;
class IMessageBus;
class IMessageContext;
class IStormSyncTransportMessageService;

using IMessageBusPtr = TSharedPtr<IMessageBus, ESPMode::ThreadSafe>;

/**
 * Implements a local endpoint for Storm Sync
 */
class FStormSyncTransportServerEndpoint : public IStormSyncTransportServerLocalEndpoint
{
public:
	FStormSyncTransportServerEndpoint();
	virtual ~FStormSyncTransportServerEndpoint() override;

	//~ Begin IStormSyncTransportServerLocalEndpoint
	/** Creates and starts tcp listener */
	virtual bool StartTcpListener(const FIPv4Endpoint& InEndpoint) override;

	/** Creates and starts tcp listener with endpoint configured in project settings */
	virtual bool StartTcpListener() override;

	/** Returns whether message endpoint is currently active */
	virtual bool IsRunning() const override;

	/** Returns whether underlying FTcpListener has been created, is currently active and listening for incoming connections */
	virtual bool IsTcpServerActive() const override;

	/** Returns endpoint address tcp server is currently listening on (ip:port), empty string otherwise (if not active / listening) */
	virtual FString GetTcpServerEndpointAddress() const override;

	/** Returns underlying message endpoint */
	virtual TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const override;
	//~ End IStormSyncTransportLocalEndpoint

	/** Initializes the message endpoint. */
	void InitializeMessaging(const FString& InEndpointFriendlyName = TEXT("Default"));
	
protected:
	/** Shutdown the message endpoint. */
	void ShutdownMessaging();

	/** Shutdown the tcp server listener. */
	void ShutdownTcpListener();

private:
	/** Our instance of tcp listener wrapper */
	TUniquePtr<FStormSyncTransportTcpServer> TcpServer;

	/**
	 * Holds a weak pointer to the message bus.
	 *
	 * @note Since we're using the default message bus, this might be not necessary
	 */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr;

	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Callback handler to receive FStormSyncTransportPingMessage messages */
	void HandlePingMessage(const FStormSyncTransportPingMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);

	/** Callback handler to receive FStormSyncTransportPongMessage messages */
	void HandlePongMessage(const FStormSyncTransportPongMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);

	/** Callback handler executed when a buffer is fully received over tcp socket and ready to be extracted  */
	void HandleReceivedTcpBuffer(const FIPv4Endpoint& InEndpoint, const TSharedPtr<FSocket>& InSocket, const FStormSyncBufferPtr& InBuffer);

	/** Callback handler to receive FStormSyncTransportSyncRequest messages */
	void HandleSyncRequestMessage(const FStormSyncTransportSyncRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext);

	/** Figures out the diffing state between remote package dependencies and local, and sends back a sync (push) response to sender to kick in the transfer process */
	void SendSyncResponse(const FStormSyncTransportSyncRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext) const;
};
