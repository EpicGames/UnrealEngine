// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncTransportServerModule.h"

#include "IStormSyncTransportCoreModule.h"
#include "IStormSyncTransportLocalEndpoint.h"
#include "MessageEndpoint.h"
#include "Misc/CoreDelegates.h"
#include "ServiceDiscovery/StormSyncDiscoveryManager.h"
#include "ServiceDiscovery/StormSyncHeartbeatEmitter.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncTransportServerEndpoint.h"
#include "StormSyncTransportServerLog.h"
#include "StormSyncTransportSettings.h"
#include "Utils/StormSyncTransportCommandUtils.h"

#define LOCTEXT_NAMESPACE "StormSyncTransportServerModule"

void FStormSyncTransportServerModule::StartupModule()
{
	const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();

	// Auto-start is disabled if running commandlet
	const bool bIsCommandLineAutoStartDisabled = IsRunningCommandlet() || UE::StormSync::Transport::Private::IsServerAutoStartDisabled();

	if (!bIsCommandLineAutoStartDisabled && Settings->IsAutoStartServer())
	{
		ExecuteStartServer({});
	}

	RegisterConsoleCommands();

	// Register for engine initialization completed so we can broadcast presence over the network and start heartbeats
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStormSyncTransportServerModule::OnPostEngineInit);

	IStormSyncTransportCoreModule::Get().OnGetCurrentTcpServerEndpointAddress().BindRaw(this, &FStormSyncTransportServerModule::GetCurrentTcpServerEndpointAddress);
	IStormSyncTransportCoreModule::Get().OnGetServerEndpointMessageAddress().BindRaw(this, &FStormSyncTransportServerModule::GetServerEndpointMessageAddressId);
}

void FStormSyncTransportServerModule::ShutdownModule()
{
	if (IStormSyncTransportCoreModule::IsAvailable())
	{
		IStormSyncTransportCoreModule::Get().OnGetCurrentTcpServerEndpointAddress().Unbind();
		IStormSyncTransportCoreModule::Get().OnGetServerEndpointMessageAddress().Unbind();
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	if (ServerEndpoint.IsValid())
	{
		ServerEndpoint.Reset();
		FStormSyncCoreDelegates::OnStormSyncServerStopped.Broadcast();
	}

	if (HeartbeatEmitter.IsValid())
	{
		HeartbeatEmitter->Exit();
	}

	if (DiscoveryManager.IsValid())
	{
		DiscoveryManager->Stop();
	}

	UnregisterConsoleCommands();
}

void FStormSyncTransportServerModule::StartDiscoveryManager()
{
	ConditionalStartHeartBeatEmitter();
	ConditionalStartDiscoveryManager();
}

void FStormSyncTransportServerModule::StartServerEndpoint(const FString& InEndpointFriendlyName)
{
	StartDiscoveryManager();
	
	if (!ServerEndpoint.IsValid())
	{
		ServerEndpoint = CreateServerLocalEndpoint(TEXT("Server"));
		if (!ServerEndpoint.IsValid())
		{
			UE_LOGF(LogStormSyncServer, Error, "FStormSyncTransportServerModule::StartServerEndpoint - Failed to create Server Local Endpoint");
			return;
		}
	}
	
	if (IsRunning())
	{
		UE_LOGF(LogStormSyncServer, Warning, "FStormSyncTransportServerModule::StartServerEndpoint - Server endpoint TCP listener already running");
		return;
	}
	
	if (ServerEndpoint->StartTcpListener() && IsRunning())
	{
		FStormSyncCoreDelegates::OnStormSyncServerStarted.Broadcast();
	}

	if (bEngineInitComplete)
	{
		PublishPingMessage();
	}
}

TSharedPtr<IStormSyncTransportServerLocalEndpoint> FStormSyncTransportServerModule::CreateServerLocalEndpoint(const FString& InEndpointFriendlyName) const
{
	TSharedPtr<FStormSyncTransportServerEndpoint, ESPMode::ThreadSafe> Endpoint = MakeShared<FStormSyncTransportServerEndpoint>();
	check(Endpoint.IsValid());
	Endpoint->InitializeMessaging(InEndpointFriendlyName);
	return Endpoint;
}

FString FStormSyncTransportServerModule::GetServerEndpointMessageAddressId() const
{
	return ServerEndpoint.IsValid() && ServerEndpoint->IsRunning() ? ServerEndpoint->GetMessageEndpoint()->GetAddress().ToString() : TEXT("");
}

FString FStormSyncTransportServerModule::GetDiscoveryManagerMessageAddressId() const
{
	return DiscoveryManager.IsValid() ? DiscoveryManager->GetMessageEndpoint()->GetAddress().ToString() : TEXT("");
}

FStormSyncHeartbeatEmitter& FStormSyncTransportServerModule::GetHeartbeatEmitter() const
{
	return *HeartbeatEmitter;
}

bool FStormSyncTransportServerModule::IsRunning() const
{
	return ServerEndpoint.IsValid() && ServerEndpoint->IsRunning() && ServerEndpoint->IsTcpServerActive();
}

bool FStormSyncTransportServerModule::GetServerStatus(FText& OutStatusText) const
{
	if (!ServerEndpoint.IsValid())
	{
		OutStatusText = LOCTEXT("ServerStatusEndpointInvalid", "Server is not active.");
		return false;
	}

	const bool bIsRunning = ServerEndpoint->IsRunning() && ServerEndpoint->IsTcpServerActive();

	if (bIsRunning)
	{
		OutStatusText = FText::Format(
			LOCTEXT("ServerStatusEndpointRunning", "Server is currently running and listening for incoming connections on {0}"),
			FText::FromString(ServerEndpoint->GetTcpServerEndpointAddress())
		);
	}
	else
	{
		OutStatusText = LOCTEXT("ServerStatusEndpointNotRunning", "Server is not running.");
	}
	
	return bIsRunning;
}

FString FStormSyncTransportServerModule::GetCurrentTcpServerEndpointAddress() const
{
	return ServerEndpoint ? ServerEndpoint->GetTcpServerEndpointAddress() : FString();
}

void FStormSyncTransportServerModule::ConditionalStartHeartBeatEmitter()
{
	if (!HeartbeatEmitter)
	{
		HeartbeatEmitter = MakeUnique<FStormSyncHeartbeatEmitter>();
	}
}

void FStormSyncTransportServerModule::ConditionalStartDiscoveryManager()
{
	if (!DiscoveryManager)
	{
		const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();

		DiscoveryManager = MakeUnique<FStormSyncDiscoveryManager>(
			Settings->GetMessageBusHeartbeatTimeout(),
			Settings->GetMessageBusTimeBeforeRemovingInactiveSource(),
			Settings->GetDiscoveryManagerTickInterval(),
			Settings->IsDiscoveryPeriodicPublishEnabled()
		);

		if (bEngineInitComplete)
		{
			PublishConnectMessage();
		}
	}
}

void FStormSyncTransportServerModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Server.Start"),
		TEXT("Starts Storm Sync Server"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportServerModule::ExecuteStartServer),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Server.Stop"),
		TEXT("Stops Storm Sync Server"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportServerModule::ExecuteStopServer),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Server.Status"),
		TEXT("Prints Storm Sync Server status"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FStormSyncTransportServerModule::ExecuteServerStatus),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Server.Debug"),
		TEXT("Prints out server address enpoint id to the console"),
		FConsoleCommandWithArgsDelegate::CreateLambda([this](const TArray<FString>& Args)
		{
			const FString AddressId = GetServerEndpointMessageAddressId();
			UE_LOGF(LogStormSyncServer, Display, "StormSync.Server.Debug - EndpointId: %ls", *AddressId);
		}),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSync.Discovery.Wakeup"),
		TEXT("Send a wakeup request through discovery manager."),
		FConsoleCommandWithArgsDelegate::CreateLambda([this](const TArray<FString>& Args)
		{
			if (DiscoveryManager)
			{
				DiscoveryManager->SendWakeUp();
			}
		}),
		ECVF_Default
	));
}

void FStormSyncTransportServerModule::UnregisterConsoleCommands()
{
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}

	ConsoleCommands.Empty();
}

void FStormSyncTransportServerModule::OnPostEngineInit()
{
	UE_LOGF(LogStormSyncServer, Verbose, "FStormSyncTransportServerModule::OnPostEngineInit - Publish ping messages for discover manager and server endpoint ...");

	bEngineInitComplete = true;
	
	// We broadcast a message to notify others about this editor instance (this is required so that further "direct" send are received on the other end)

	// For service discovery
	PublishConnectMessage();

	// For server endpoint
	PublishPingMessage();
}

void FStormSyncTransportServerModule::PublishConnectMessage() const
{
	if (DiscoveryManager.IsValid())
	{
		DiscoveryManager->PublishConnectMessage();
	}
}

void FStormSyncTransportServerModule::PublishPingMessage() const
{
	if (ServerEndpoint.IsValid())
	{
		const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint = ServerEndpoint->GetMessageEndpoint();
		TUniquePtr<FStormSyncTransportPingMessage> Message(FMessageEndpoint::MakeMessage<FStormSyncTransportPingMessage>());
		if (MessageEndpoint.IsValid() && Message.IsValid())
		{
			MessageEndpoint->Publish(Message.Release());
		}
	}
}

void FStormSyncTransportServerModule::ExecuteStartServer(const TArray<FString>& InArgs)
{
	StartServerEndpoint(TEXT("Server"));
}

void FStormSyncTransportServerModule::ExecuteStopServer(const TArray<FString>& InArgs)
{
	if (ServerEndpoint.IsValid())
	{
		ServerEndpoint.Reset();
		FStormSyncCoreDelegates::OnStormSyncServerStopped.Broadcast();
	}
	else
	{
		UE_LOGF(LogStormSyncServer, Warning, "FStormSyncTransportServerModule::ExecuteStopServer - Server endpoint already inactive");
	}
}

void FStormSyncTransportServerModule::ExecuteServerStatus(const TArray<FString>& InArgs) const
{
	FText StatusText;
	GetServerStatus(StatusText);

	UE_LOGF(LogStormSyncServer, Display, "FStormSyncTransportServerModule::ExecuteServerStatus - %ls", *StatusText.ToString());
}

IMPLEMENT_MODULE(FStormSyncTransportServerModule, StormSyncTransportServer)

#undef LOCTEXT_NAMESPACE
