// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportServerModule.h"

class FStormSyncDiscoveryManager;

/**
 * Implements the StormSyncTransportServer module.
 */
class FStormSyncTransportServerModule : public IStormSyncTransportServerModule
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//~ Begin IStormSyncTransportServerModule
	virtual void StartDiscoveryManager() override;
	virtual void StartServerEndpoint(const FString& InEndpointFriendlyName) override;
	virtual TSharedPtr<IStormSyncTransportServerLocalEndpoint> CreateServerLocalEndpoint(const FString& InEndpointFriendlyName) const override;
	virtual FString GetServerEndpointMessageAddressId() const override;
	virtual FString GetDiscoveryManagerMessageAddressId() const override;
	virtual FStormSyncHeartbeatEmitter& GetHeartbeatEmitter() const override;
	virtual bool IsRunning() const override;
	virtual bool GetServerStatus(FText& OutStatusText) const override;
	//~ End IStormSyncTransportServerModule

private:
	/** Returns the current address (ip:port) the tcp server is bound to. */
	FString GetCurrentTcpServerEndpointAddress() const;
	
	/** Starts the heart beat emitter if it was not started already. */
	void ConditionalStartHeartBeatEmitter();

	/** Starts the discovery manager if it was not started already. */
	void ConditionalStartDiscoveryManager();
	
	/** Called from StartupModule and sets up console commands for the plugin via IConsoleManager */
	void RegisterConsoleCommands();

	/** Called from ShutdownModule and clears out previously registered console commands */
	void UnregisterConsoleCommands();

	/** Event handler to kick in operations once engine is fully initialized (to publish a connect message) */
	void OnPostEngineInit();

	/** Notify network this client is ready by sending a connect message and starting heartbeats */
	void PublishConnectMessage() const;

	/** Publishes the ping message for the server endpoint. */
	void PublishPingMessage() const;

	/** Command handler for "StormSync.Server.Start" */
	void ExecuteStartServer(const TArray<FString>& InArgs);
	
	/** Command handler for "StormSync.Server.Stop" */
	void ExecuteStopServer(const TArray<FString>& InArgs);
	
	/** Command handler for "StormSync.Server.Status" */
	void ExecuteServerStatus(const TArray<FString>& InArgs) const;

private:
	/** Our message endpoint provider */
	TSharedPtr<IStormSyncTransportServerLocalEndpoint, ESPMode::ThreadSafe> ServerEndpoint;

	/** Unique ptr to our Heartbeat Emitter (Runnable sending heartbeat messages at a fixed interval to subscribed recipients) */
	TUniquePtr<FStormSyncHeartbeatEmitter> HeartbeatEmitter;
	
	/** Unique ptr to our Discovery Manager */
	TUniquePtr<FStormSyncDiscoveryManager> DiscoveryManager;

	/** References of registered console commands via IConsoleManager */
	TArray<IConsoleObject*> ConsoleCommands;

	/** Indicates when the engine init is complete. This is used to know when pending messages can be published. */
	bool bEngineInitComplete = false;
};
