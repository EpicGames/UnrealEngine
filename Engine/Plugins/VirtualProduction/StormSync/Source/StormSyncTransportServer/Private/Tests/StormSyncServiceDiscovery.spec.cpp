// Copyright Epic Games, Inc. All Rights Reserved.

#include "MessageEndpointBuilder.h"
#include "Misc/AutomationTest.h"
#include "StormSyncCoreDelegates.h"
#include "StormSyncTransportMessages.h"
#include "StormSyncTransportSettings.h"
#include "ServiceDiscovery/StormSyncDiscoveryManager.h"
#include "ServiceDiscovery/StormSyncHeartbeatEmitter.h"

/**
 * Mock version of StormSyncDiscoveryManager
 *
 * Only listens for heartbeat messages and calls up a delegate for each incoming message
 */
class FMockDiscoveryManager
{
public:
	FMockDiscoveryManager()
	{
		MessageEndpoint = FMessageEndpoint::Builder(TEXT("StormSyncMessageHeartbeatManager (MockDiscoveryManager)"))
			.ReceivingOnThread(ENamedThreads::GameThread)
			.Handling<FStormSyncTransportHeartbeatMessage>(this, &FMockDiscoveryManager::HandleHeartbeatMessage);
	}

	~FMockDiscoveryManager()
	{
		// Disable the Endpoint message handling since the message could keep it alive a bit.
		if (MessageEndpoint)
		{
			MessageEndpoint->Disable();
			MessageEndpoint.Reset();
		}
	}

	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> GetMessageEndpoint() const
	{
		return MessageEndpoint;
	}

	DECLARE_DELEGATE_TwoParams(FOnHeartbeatMessage, const FStormSyncTransportHeartbeatMessage&, const FMessageAddress&);
	FOnHeartbeatMessage OnHeartbeatMessage;

private:
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Callback handler to receive FStormSyncTransportHeartbeatMessage messages */
	void HandleHeartbeatMessage(const FStormSyncTransportHeartbeatMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
	{
		if (OnHeartbeatMessage.IsBound())
		{
			OnHeartbeatMessage.Execute(InMessage, MessageContext->GetSender());
		}
	}
};

BEGIN_DEFINE_SPEC(FStormSyncServiceDiscoverySpec, "StormSync.StormSyncTransportServer.StormSyncServiceDiscovery", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	/** Mock version of StormSyncDiscoveryManager (used in "StormSyncHeartbeatEmitter" suite) */
	TUniquePtr<FMockDiscoveryManager> MockDiscoveryManager;

	/** Unique ptr to our Discovery Manager that is going to be tested in "StormSyncDiscoveryManager" suite */
	TUniquePtr<FStormSyncDiscoveryManager> DiscoveryManager;

	/**
	 * Slightly lower inactive timeout than the default, so that tests don't take that much time (15s is a bit much)
	 * And no lower than 2, since it's the default responsive timeout (heartbeat timeout)
	 */
	const double TestInactiveTimeout = 5.0;

	/** Message endpoint for suite that needs sending */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MockMessageEndpoint;

	/** Unique ptr to our Heartbeat Emitter (Runnable sending heartbeat messages at a fixed interval to subscribed recipients) */
	TUniquePtr<FStormSyncHeartbeatEmitter> HeartbeatEmitter;

	int32 HeartbeatCounts = 0;
	const int32 MaxHeartbeats = 2;

	double StartTime = 0;

	bool bConnectionDetected = false;
	bool bConnectionStateChangeDetected = false;

	float ErrorTolerance;

	TArray<FDelegateHandle> DelegateHandles;

END_DEFINE_SPEC(FStormSyncServiceDiscoverySpec)

void FStormSyncServiceDiscoverySpec::Define()
{
	ErrorTolerance = GetDefault<UStormSyncTransportSettings>()->GetMessageBusHeartbeatPeriod() + 1.f;
	
	Describe(TEXT("StormSyncHeartbeatEmitter"), [this]()
	{
		BeforeEach([this]()
		{
			MockDiscoveryManager = MakeUnique<FMockDiscoveryManager>();
			HeartbeatEmitter = MakeUnique<FStormSyncHeartbeatEmitter>();

			StartTime = FApp::GetCurrentTime();
		});

		// Wait for 2 heartbeats before exiting this test
		LatentIt(TEXT("should emit heartbeats at a regular interval"), [this](const FDoneDelegate& Done)
		{
			const TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MockEndpoint = MockDiscoveryManager->GetMessageEndpoint();

			AddInfo(FString::Printf(TEXT("Test starting Heartbeat for %s"), *MockEndpoint->GetAddress().ToString()));
			HeartbeatEmitter->StartHeartbeat(MockEndpoint->GetAddress(), MockEndpoint);
			AddInfo(FString::Printf(TEXT("Test started Heartbeat for %s"), *MockEndpoint->GetAddress().ToString()));

			float HeartbeatPeriod = GetDefault<UStormSyncTransportSettings>()->GetMessageBusHeartbeatPeriod();
			
			MockDiscoveryManager->OnHeartbeatMessage.BindLambda([MockEndpoint, HeartbeatPeriod, Done, this](const FStormSyncTransportHeartbeatMessage& InMessage, const FMessageAddress& InAddress)
			{
				++HeartbeatCounts;

				const double CurrentTime = FApp::GetCurrentTime();
				const double ElapsedTime = CurrentTime - StartTime;

				AddInfo(FString::Printf(TEXT("Got Lambda Heartbeat from %s (Time: %f)"), *InAddress.ToString(), ElapsedTime));
				AddInfo(FString::Printf(TEXT("Total Heartbeats %d"), HeartbeatCounts));

				if (HeartbeatCounts == MaxHeartbeats)
				{
					HeartbeatEmitter->StopHeartbeat(MockEndpoint->GetAddress(), MockEndpoint);
					
					const float MinValue = (MaxHeartbeats - 1) * HeartbeatPeriod;
					const float MaxValue = (MaxHeartbeats + 1) * HeartbeatPeriod;
					
					// With a default interval value of 1s in settings, it should be higher than 5s but lower than 6s
					const bool bIsWithin = FMath::IsWithin(ElapsedTime, MinValue, MaxValue);
					AddInfo(FString::Printf(TEXT("Test %f is within %f and %f"), ElapsedTime, MinValue, MaxValue));
					TestTrue(TEXT("With a default interval value of 1s in settings, it should be higher than 5s but lower than 6s"), bIsWithin);

					Done.Execute();
				}
			});
		});

		AfterEach([this]()
		{
			HeartbeatCounts = 0;
			MockDiscoveryManager->OnHeartbeatMessage.Unbind();
			MockDiscoveryManager.Reset();
			HeartbeatEmitter->Exit();
			HeartbeatEmitter.Reset();
		});
	});

	// This test is not really unit testing, using here the discovery manager spawned in background by server module, and checking
	// behavior on an incoming connection. Stuff tested:
	//
	// - is first detected as a new connection
	// - its state is then detected as unresponsive (since no heartbeat handling for this mock endpoint)
	// - is then cleaned up after the inactive timeout threshold
	Describe(TEXT("StormSyncDiscoveryManager"), [this]()
	{
		// I wish there was a simple "Before" hook, we just want to publish the connect message once for all these tests in this suite
		BeforeEach([this]()
		{
			const UStormSyncTransportSettings* Settings = GetDefault<UStormSyncTransportSettings>();
			DiscoveryManager = MakeUnique<FStormSyncDiscoveryManager>(
				Settings->GetMessageBusHeartbeatTimeout(),
				TestInactiveTimeout,
				Settings->GetDiscoveryManagerTickInterval(),
				Settings->IsDiscoveryPeriodicPublishEnabled()
			);
			MockMessageEndpoint = FMessageEndpoint::Builder(TEXT("MockMessageEndpoint (StormSyncDiscoveryManager Test Suite)"));
			StartTime = FApp::GetCurrentTime();

			FStormSyncTransportConnectMessage* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportConnectMessage>();
			MockMessageEndpoint->Send(Message, DiscoveryManager->GetMessageEndpoint()->GetAddress());

			AddInfo(FString::Printf(TEXT("Publish before each")));
		});
		
		LatentIt(TEXT("Should detect new connection, state change and disconnection"), [this](const FDoneDelegate& Done)
		{
			{
				FMessageAddress MockMessageAddress = MockMessageEndpoint->GetAddress();
				AddInfo(FString::Printf(TEXT("Mock Address is %s"), *MockMessageAddress.ToString()));

				const FDelegateHandle Handle = FStormSyncCoreDelegates::OnServiceDiscoveryConnection.AddLambda([this, Done, MockMessageAddress](const FString InAddress, const FStormSyncConnectedDevice& ConnectedDevice)
				{
					if (InAddress != MockMessageAddress.ToString())
					{
						// Just ignore those that are not from our mock endpoint
						return;
					}

					bConnectionDetected = true;

					AddInfo(FString::Printf(TEXT("Incoming connection from %s (Mock Address is %s)"), *InAddress, *MockMessageAddress.ToString()));
					TestEqual(TEXT("Should detect incoming connections"), InAddress, MockMessageAddress.ToString());
				});

				DelegateHandles.Add(Handle);
			}

			{
				FMessageAddress MockMessageAddress = MockMessageEndpoint->GetAddress();
				AddInfo(FString::Printf(TEXT("Mock Address is %s"), *MockMessageAddress.ToString()));
				
				const FDelegateHandle Handle = FStormSyncCoreDelegates::OnServiceDiscoveryStateChange.AddLambda([this, Done, MockMessageAddress](const FString InAddress, const EStormSyncConnectedDeviceState InState)
				{
					if (InAddress != MockMessageAddress.ToString())
					{
						// Just ignore those that are not from our mock endpoint
						return;
					}

					bConnectionStateChangeDetected = true;

					AddInfo(FString::Printf(TEXT("Detected state change for %s (Mock Address is %s)"), *InAddress, *MockMessageAddress.ToString()));
					TestEqual(TEXT("Should detect unresponsive state"), InState, EStormSyncConnectedDeviceState::State_Unresponsive);
					
					const double CurrentTime = FApp::GetCurrentTime();
					const double ElapsedTime = CurrentTime - StartTime;

					// With a default timeout value of 2s in settings, elapsed time should be around this value
					const double HeartbeatTimeout = GetDefault<UStormSyncTransportSettings>()->GetMessageBusHeartbeatTimeout();
					const bool bIsNearlyEqual = FMath::IsNearlyEqual(ElapsedTime, HeartbeatTimeout, ErrorTolerance);
					AddInfo(FString::Printf(TEXT("Test %f is roughly %f"), ElapsedTime, HeartbeatTimeout));
					TestTrue(TEXT("With a default heartbeat tiemout of 2s in settings, elapsed time should be around this value"), bIsNearlyEqual);
				});
				
				DelegateHandles.Add(Handle);
			}

			{
				FMessageAddress MockMessageAddress = MockMessageEndpoint->GetAddress();
				AddInfo(FString::Printf(TEXT("Mock Address is %s"), *MockMessageAddress.ToString()));
							
				const FDelegateHandle Handle = FStormSyncCoreDelegates::OnServiceDiscoveryDisconnection.AddLambda([this, Done, MockMessageAddress](const FString InAddress)
				{
					if (InAddress != MockMessageAddress.ToString())
					{
						// Just ignore those that are not from our mock endpoint
						return;
					}

					AddInfo(FString::Printf(TEXT("Detected disconnection for %s (Mock Address is %s)"), *InAddress, *MockMessageAddress.ToString()));
					
					const double CurrentTime = FApp::GetCurrentTime();
					const double ElapsedTime = CurrentTime - StartTime;

					const bool bIsNearlyEqual = FMath::IsNearlyEqual(ElapsedTime, TestInactiveTimeout, ErrorTolerance);
					AddInfo(FString::Printf(TEXT("Test %f is roughly %f"), ElapsedTime, TestInactiveTimeout));
					TestTrue(TEXT("With a default inactive tiemout of 15s in settings, elapsed time should be around this value"), bIsNearlyEqual);
					TestTrue(TEXT("Connection was detected"), bConnectionDetected);
					TestTrue(TEXT("State change was detected"), bConnectionStateChangeDetected);

					Done.Execute();
				});

				DelegateHandles.Add(Handle);
			}
		});

		AfterEach([this]()
		{
			for (const FDelegateHandle& DelegateHandle : DelegateHandles)
			{
				FStormSyncCoreDelegates::OnServiceDiscoveryConnection.Remove(DelegateHandle);
				FStormSyncCoreDelegates::OnServiceDiscoveryStateChange.Remove(DelegateHandle);
				FStormSyncCoreDelegates::OnServiceDiscoveryDisconnection.Remove(DelegateHandle);
			}

			DelegateHandles.Empty();

			DiscoveryManager.Reset();

			MockMessageEndpoint->Disable();
			MockMessageEndpoint.Reset();
		});
	});
}
