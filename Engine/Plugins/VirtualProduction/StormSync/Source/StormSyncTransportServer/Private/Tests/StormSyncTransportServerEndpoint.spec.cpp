// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/TcpSocketBuilder.h"
#include "Dom/JsonObject.h"
#include "HAL/FileManager.h"
#include "IStormSyncTransportServerModule.h"
#include "MessageEndpointBuilder.h"
#include "Misc/AutomationTest.h"
#include "Misc/PackageName.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "StormSyncTransportServerEndpoint.h"
#include "StormSyncTransportSettings.h"

BEGIN_DEFINE_SPEC(FStormSyncTransportServerEndpointSpec, "StormSync.StormSyncTransportServer.StormSyncTransportServerEndpoint", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	static constexpr const TCHAR* EndpointFriendlyName = TEXT("Test Suite ServerEndpoint (Message System)");

	TSharedPtr<IStormSyncTransportServerLocalEndpoint> ServerEndpoint;

	TSharedPtr<FMessageEndpoint> LocalMessageEndpoint;

	/** Used in the "TCP Server" suite to create the transport tcp server and local socket to connect from */
	FIPv4Endpoint TestServerEndpoint;

	FSocket* LocalSocket;

	/** Default to roughly 4 Mb. Socket Send / Receive Buffer Size */
	const int32 SocketBufferSize = 4 * 1024 * 1024;

	TDelegate<void(const FStormSyncTransportPongMessage&)> OnPongMessage;

	TDelegate<void(const FStormSyncTransportPushResponse&)> OnPushResponse;

	void HandlePongMessage(const FStormSyncTransportPongMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&)
	{
		OnPongMessage.ExecuteIfBound(InMessage);
	}

	void HandlePushResponse(const FStormSyncTransportPushResponse& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&)
	{
		OnPushResponse.ExecuteIfBound(InMessage);
	}

	static uint64 GetMockFileSize(const FName& InPackageName)
	{
		uint64 FileSize = 0;
		FString PackageFilepath;
		if (FPackageName::DoesPackageExist(InPackageName.ToString(), &PackageFilepath))
		{
			if (const TUniquePtr<FArchive> FileHandle = TUniquePtr<FArchive>(IFileManager::Get().CreateFileReader(*PackageFilepath)))
			{
				FileSize = FileHandle->TotalSize();
				FileHandle->Close();
			}
		}
		
		return FileSize;
	}

END_DEFINE_SPEC(FStormSyncTransportServerEndpointSpec)

void FStormSyncTransportServerEndpointSpec::Define()
{
	Describe(TEXT("Server Endpoint initialization"), [this]()
	{
		It(TEXT("should initialize properly using endpoint ServerModule interface CreateServerLocalEndpoint"), [this]()
		{
			const uint16 TestServerPort = GetDefault<UStormSyncTransportSettings>()->GetTcpServerPort() + 1;
			const FIPv4Endpoint Endpoint = FIPv4Endpoint(FIPv4Address::InternalLoopback, TestServerPort);
			
			ServerEndpoint = IStormSyncTransportServerModule::Get().CreateServerLocalEndpoint(EndpointFriendlyName);
			ServerEndpoint->StartTcpListener(Endpoint);
			
			TestTrue(TEXT("is running"), ServerEndpoint->IsRunning());
			TestTrue(TEXT("tcp server is active"), ServerEndpoint->IsTcpServerActive());
		});

		AfterEach([this]()
		{
			if (ServerEndpoint.IsValid())
			{
				ServerEndpoint.Reset();
			}
		});
		
	});
	
	Describe(TEXT("Message System"), [this]()
	{
		BeforeEach([this]()
		{
			ServerEndpoint = IStormSyncTransportServerModule::Get().CreateServerLocalEndpoint(EndpointFriendlyName);
			
			const uint16 TestServerPort = GetDefault<UStormSyncTransportSettings>()->GetTcpServerPort() + 1;
			const FIPv4Endpoint Endpoint = FIPv4Endpoint(FIPv4Address::InternalLoopback, TestServerPort);
			ServerEndpoint->StartTcpListener(Endpoint);
			
			LocalMessageEndpoint = FMessageEndpoint::Builder(EndpointFriendlyName)
				.ReceivingOnThread(ENamedThreads::Type::GameThread)
				.Handling<FStormSyncTransportPushResponse>(this, &FStormSyncTransportServerEndpointSpec::HandlePushResponse)
				.Handling<FStormSyncTransportPongMessage>(this, &FStormSyncTransportServerEndpointSpec::HandlePongMessage);
		});

		It(TEXT("should be running"), [this]()
		{
			TestTrue(TEXT("is running"), ServerEndpoint->IsRunning());
			TestTrue(TEXT("tcp server is active"), ServerEndpoint->IsTcpServerActive());
		});

		It(TEXT("has correct endpoint friendly name"), [this]()
		{
			const TSharedPtr<FMessageEndpoint> MessageEndpoint = ServerEndpoint->GetMessageEndpoint();
			if (!MessageEndpoint.IsValid())
			{
				AddError(FString::Printf(TEXT("Message endpoint is invalid")));
				return;
			}

			const FString DebugName = MessageEndpoint->GetDebugName().ToString();
			TestTrue(TEXT("Message Endpoint includes expected friendly name"), DebugName.Contains(EndpointFriendlyName));
		});

		LatentIt(TEXT("should reply with a pong message"), [this](const FDoneDelegate& Done)
		{
			const TSharedPtr<FMessageEndpoint> ServerMessageEndpoint = ServerEndpoint->GetMessageEndpoint();
			if (!ServerMessageEndpoint.IsValid())
			{
				AddError(FString::Printf(TEXT("Message endpoint is invalid")));
				Done.Execute();
				return;
			}
			
			LocalMessageEndpoint->Send(FMessageEndpoint::MakeMessage<FStormSyncTransportPingMessage>(), ServerMessageEndpoint->GetAddress());

			OnPongMessage.BindLambda([this, Done] (const FStormSyncTransportPongMessage& Message)
			{
				TestEqual(TEXT("Should have hostname"), Message.Hostname, FPlatformProcess::ComputerName());
				TestEqual(TEXT("Should have username"), Message.Username, FPlatformProcess::UserName());
				Done.Execute();
			});
		});

		LatentIt(TEXT("should send back sync response"), [this](const FDoneDelegate& Done)
		{
			const TSharedPtr<FMessageEndpoint> ServerMessageEndpoint = ServerEndpoint->GetMessageEndpoint();
			if (!ServerMessageEndpoint.IsValid())
			{
				AddError(FString::Printf(TEXT("Message endpoint is invalid")));
				Done.Execute();
				return;
			}

			TArray<FName> PackageNames;
			FStormSyncPackageDescriptor PackageDescriptor;
			FStormSyncTransportSyncRequest* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportSyncRequest>(PackageNames, PackageDescriptor);
			check(Message);
			
			LocalMessageEndpoint->Send(Message, ServerMessageEndpoint->GetAddress());

			OnPushResponse.BindLambda([this, Done](const FStormSyncTransportPushResponse& Response)
			{
				const FString ServerEndpointAddress = GetDefault<UStormSyncTransportSettings>()->GetServerEndpoint();
				TestEqual(TEXT("Should have hostname"), Response.HostName, FPlatformProcess::ComputerName());
				TestEqual(TEXT("Should have address"), Response.HostAddress, ServerEndpointAddress);
				// Since we don't send any file dependencies to check against
				TestEqual(TEXT("Should have no modifiers"), Response.Modifiers.Num(), 0);
				Done.Execute();
			});
		});

		LatentIt(TEXT("should send back sync response with proper file modifier"), [this](const FDoneDelegate& Done)
		{
			const TSharedPtr<FMessageEndpoint> ServerMessageEndpoint = ServerEndpoint->GetMessageEndpoint();
			if (!ServerMessageEndpoint.IsValid())
			{
				AddError(FString::Printf(TEXT("Message endpoint is invalid")));
				Done.Execute();
				return;
			}
			
			// Setup some mock dependency remote will have to check against
			// Only PackageName, FileSize and FileHash are relevant when computing diff
			FStormSyncFileDependency FileDependency;
			FileDependency.PackageName = TEXT("/StormSync/Fixtures/T_Fixture_Noise_Background");
			FileDependency.FileSize = GetMockFileSize(FileDependency.PackageName);
			FileDependency.FileHash = TEXT("__dirty__");
			
			TArray<FName> PackageNames;
			FStormSyncPackageDescriptor PackageDescriptor;
			PackageNames.Add(FileDependency.PackageName);
			PackageDescriptor.Dependencies.Add(FileDependency);
			
			FStormSyncTransportSyncRequest* Message = FMessageEndpoint::MakeMessage<FStormSyncTransportSyncRequest>(PackageNames, PackageDescriptor);
			check(Message);

			LocalMessageEndpoint->Send(Message, ServerMessageEndpoint->GetAddress());

			OnPushResponse.BindLambda([this, FileDependency, Done](const FStormSyncTransportPushResponse& Response)
			{
				const FString ServerEndpointAddress = GetDefault<UStormSyncTransportSettings>()->GetServerEndpoint();
				TestEqual(TEXT("Should have hostname"), Response.HostName, FPlatformProcess::ComputerName());
				TestEqual(TEXT("Should have address"), Response.HostAddress, ServerEndpointAddress);
				TestEqual(TEXT("Should have one modifiers"), Response.Modifiers.Num(), 1);

				if (!Response.Modifiers.IsValidIndex(0))
				{
					AddError(TEXT("Invalid modifier"));
					Done.Execute();
					return;
				}

				const FStormSyncFileModifierInfo ModifierInfo = Response.Modifiers[0];
				TestEqual(TEXT("Modifier info operation"), ModifierInfo.ModifierOperation, EStormSyncModifierOperation::Overwrite);
				TestEqual(TEXT("Modifier info PackageName"), ModifierInfo.FileDependency.PackageName, FileDependency.PackageName);
				TestEqual(TEXT("Modifier info FileSize"), ModifierInfo.FileDependency.FileSize, FileDependency.FileSize);

				// Hash should be different since it's what we purposely changed on mock
				TestNotEqual(TEXT("Modifier info FileHash"), ModifierInfo.FileDependency.FileHash, FileDependency.FileHash);

				// Timestamp should be valid since it should match the actual file on disk (and mock didn't initialized)
				const FDateTime Timestamp = FDateTime::FromUnixTimestamp(ModifierInfo.FileDependency.Timestamp);
				TestTrue(TEXT("Modifier info Timestamp is initialized"), ModifierInfo.FileDependency.Timestamp != 0);
				TestTrue(TEXT("Modifier info Timestamp is not min value"), Timestamp != FDateTime::MinValue());
				TestTrue(TEXT("Modifier info Timestamp is between min value and now"), Timestamp > FDateTime::MinValue() && Timestamp < FDateTime::Now());
				
				Done.Execute();
			});
		});

		AfterEach([this]()
		{
			ServerEndpoint.Reset();
			LocalMessageEndpoint->Disable();
			LocalMessageEndpoint.Reset();
		});
	});

	Describe(TEXT("TCP Server"), [this]()
	{
		BeforeEach([this]()
		{
			ServerEndpoint = MakeShared<FStormSyncTransportServerEndpoint>();
			const uint16 TestServerPort = GetDefault<UStormSyncTransportSettings>()->GetTcpServerPort() + 1;
			const FIPv4Endpoint Endpoint = FIPv4Endpoint(FIPv4Address::InternalLoopback, TestServerPort);
			ServerEndpoint->StartTcpListener(Endpoint);
			
			LocalSocket = FTcpSocketBuilder(TEXT("MockSocket.StormSyncTransportServerEndpoint"))
				.WithSendBufferSize(SocketBufferSize)
				.WithReceiveBufferSize(SocketBufferSize);

			// Force address to be localhost
			TestServerEndpoint.Address = FIPv4Address::InternalLoopback;
			TestServerEndpoint.Port = TestServerPort;
		});
		
		It(TEXT("Should have tcp server running and active"), [this]()
		{
			TestTrue(TEXT("tcp server is running"), ServerEndpoint->IsTcpServerActive());
		});

		It(TEXT("Should receive incoming connections and respond state packet back"), [this]()
		{
			TestTrue(TEXT("tcp server is running"), ServerEndpoint->IsTcpServerActive());

			const bool bConnectStatus = LocalSocket->Connect(TestServerEndpoint.ToInternetAddr().Get());
			TestTrue(TEXT("local socket connect ok"), bConnectStatus);

			TestNotEqual(TEXT("Connection State is ok"), LocalSocket->GetConnectionState(), SCS_ConnectionError);
			
			// Block waiting for some data
			if (!LocalSocket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(1.0)))
			{
				AddError(TEXT("Connect wait for state packet failed."));
				return;
			}

			uint32 PendingDataSize = 0;
			if (LocalSocket->HasPendingData(PendingDataSize))
			{
				TArray<uint8> Buffer;
				Buffer.AddUninitialized(PendingDataSize);
				int32 BytesRead = 0;
				if (!LocalSocket->Recv(Buffer.GetData(), PendingDataSize, BytesRead, ESocketReceiveFlags::None))
				{
					AddError(TEXT("Error while receiving data via mock endpoint"));
					return;
				}

				TArray<uint8> MessageBuffer;
				for (int32 i = 0; i < BytesRead; ++i)
				{
					MessageBuffer.Add(Buffer[i]);
					if (Buffer[i] == '\x00')
					{
						const FString Message(UTF8_TO_TCHAR(MessageBuffer.GetData()));
						AddInfo(FString::Printf(TEXT("IncomingMessage from %s - Message: %s"), *TestServerEndpoint.ToString(), *Message));
						MessageBuffer.Empty();

						TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(Message);
						
						TSharedPtr<FJsonObject> JsonData;
						
						if (!FJsonSerializer::Deserialize(Reader, JsonData))
						{
							AddError(TEXT("Error parsing incoming message"));
							return;
						}
						
						TSharedPtr<FJsonValue> CommandField = JsonData->TryGetField(TEXT("command"));
						
						if (!CommandField.IsValid())
						{
							AddError(TEXT("Error parsing incoming message - invalid format"));
							return;
						}
						
						TestEqual(TEXT("Command is the correct one"), CommandField->AsString(), TEXT("state"));
					}
				}
			}
		});

		AfterEach([this]()
		{
			ServerEndpoint.Reset();
			LocalSocket->Close();
			ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(LocalSocket);
		});
		
	});
}

#undef DEFINE_TEST_MESSAGE_HANDLER
