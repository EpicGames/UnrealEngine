// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/TcpSocketBuilder.h"
#include "Misc/AutomationTest.h"
#include "Socket/StormSyncTransportTcpServer.h"
#include "StormSyncTransportSettings.h"


BEGIN_DEFINE_SPEC(FStormSyncTransportTcpServerSpec, "StormSync.StormSyncTransportServer.StormSyncTransportTcpServer", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	/** Our instance of tcp listener wrapper */
	TUniquePtr<FStormSyncTransportTcpServer> MockTcpServer;
	FIPv4Endpoint MockEndpoint;

	FSocket* MockLocalSocket;

	/** Default to roughly 4 Mb. Socket Send / Receive Buffer Size */
	const int32 SocketBufferSize = 4 * 1024 * 1024;

END_DEFINE_SPEC(FStormSyncTransportTcpServerSpec)

void FStormSyncTransportTcpServerSpec::Define()
{
	Describe(TEXT("Should receive buffers and notify back when fully received"), [this]()
	{
		BeforeEach([this]()
		{
			const FString ServerEndpoint = GetDefault<UStormSyncTransportSettings>()->GetServerEndpoint();
			const uint32 InactiveTimeoutSeconds = GetDefault<UStormSyncTransportSettings>()->GetInactiveTimeoutSeconds();

			if (!FIPv4Endpoint::Parse(ServerEndpoint, MockEndpoint))
			{
				AddError(TEXT("Failed to parse endpoint"));
				return;
			}
			
			// Our local socket for sending buffers
			MockLocalSocket = FTcpSocketBuilder(TEXT("MockSocket.StormSyncTransportTcpServer"))
				.WithSendBufferSize(SocketBufferSize)
				.WithReceiveBufferSize(SocketBufferSize);

			// Force address to be localhost
			MockEndpoint.Address = FIPv4Address::InternalLoopback;

			// Create mock server and start listening
			MockTcpServer = MakeUnique<FStormSyncTransportTcpServer>(MockEndpoint.Address, MockEndpoint.Port, InactiveTimeoutSeconds);
			if (!MockTcpServer->StartListening())
			{
				AddError(TEXT("StormSyncTransportTcpServer init failure"));
			}
		});

		It(TEXT("Should be listening for incoming connections"), [this]()
		{
			TestTrue(TEXT("IsActive()"), MockTcpServer->IsActive());
			TestEqual(TEXT("GetEndpointAddress()"), MockTcpServer->GetEndpointAddress(), FString::Printf(TEXT("127.0.0.1:%d"), MockEndpoint.Port));
		});

		LatentIt(TEXT("Should handle incoming buffer and notify back when fully received"), [this](const FDoneDelegate& Done)
		{			
			MockTcpServer->OnReceivedBuffer().AddLambda([this, Done](const FIPv4Endpoint& InEndpoint, const TSharedPtr<FSocket>& InSocket, const FStormSyncBufferPtr& InBuffer)
			{
				AddInfo(FString::Printf(TEXT("Received Buffer from %s - Buffer Size: %lld"), *InEndpoint.ToString(), InBuffer->Num()));
				Done.Execute();
			});

			const bool bConnectStatus = MockLocalSocket->Connect(MockEndpoint.ToInternetAddr().Get());
			TestTrue(TEXT("local socket connect ok"), bConnectStatus);
			
			// See if we're writable
			if (!MockLocalSocket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(2.0)))
			{
				AddError(TEXT("Local mock socket not writable"));
				Done.Execute();
				return;
			}

			uint32 DummyBufferSize = 1024;
			TArray<uint8> DummyBuffer;
			DummyBuffer.Reserve(sizeof(uint32) + DummyBufferSize);
			
			// Prepend size to the buffer
			DummyBuffer.Append(reinterpret_cast<uint8*>(&DummyBufferSize), sizeof(uint32));
			DummyBuffer.AddUninitialized(DummyBufferSize);
			
			AddInfo(FString::Printf(TEXT("About to send to buffer of size %d to %s"), DummyBuffer.Num(), *MockEndpoint.ToString()));

			int32 BytesSent = 0;
			if (!MockLocalSocket->Send(DummyBuffer.GetData(), DummyBuffer.Num(), BytesSent))
			{
				AddError(TEXT("Local mock socket failed to send dummy buffer"));
				Done.Execute();
			}
		});

		AfterEach([this]()
		{
			MockTcpServer->OnReceivedBuffer().RemoveAll(this);
			MockTcpServer.Reset();
		});
	});
}
