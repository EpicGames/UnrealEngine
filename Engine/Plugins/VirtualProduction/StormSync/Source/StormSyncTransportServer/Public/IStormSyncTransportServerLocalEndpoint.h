// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStormSyncTransportLocalEndpoint.h"

struct FIPv4Endpoint;

/** Local Endpoint interface for Storm Sync Server */
class IStormSyncTransportServerLocalEndpoint : public IStormSyncTransportLocalEndpoint
{
public:
	/** Returns endpoint address tcp server is currently listening on (ip:port), empty string otherwise (if not active / listening) */
	virtual FString GetTcpServerEndpointAddress() const = 0;

	/** Returns whether underlying FTcpListener has been created, is currently active and listening for incoming connections */
	virtual bool IsTcpServerActive() const = 0;

	/** Start TCP server and start listening with passed in endpoint */
	virtual bool StartTcpListener(const FIPv4Endpoint& InEndpoint) = 0;
	
	/** Start TCP server and start listening with configured endpoint in project settings */
	virtual bool StartTcpListener() = 0;
};
