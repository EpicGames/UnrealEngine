// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FStormSyncHeartbeatEmitter;
class IStormSyncTransportServerLocalEndpoint;

class IStormSyncTransportServerModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IStormSyncTransportServerModule& Get()
	{
		static const FName ModuleName = "StormSyncTransportServer";
		return FModuleManager::LoadModuleChecked<IStormSyncTransportServerModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable()
	{
		static const FName ModuleName = "StormSyncTransportServer";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/**
	 * Starts the discovery manager.
	 * This is necessary for clients and servers to see each other.
	 * It must be called even if the server endpoint is not created.
	 */
	virtual void StartDiscoveryManager() = 0;

	/**
	 * Starts a local transport endpoint.
	 * This will automatically start the discovery manager.
	 */
	virtual void StartServerEndpoint(const FString& InEndpointFriendlyName) = 0;
	
	/**
	 * Creates a local transport endpoint.
	 * @remark Should be used for tests only.
	 */
	virtual TSharedPtr<IStormSyncTransportServerLocalEndpoint> CreateServerLocalEndpoint(const FString& InEndpointFriendlyName) const = 0;

	/** Returns Message Address UID for server endpoint if it is currently running, empty string otherwise */
	virtual FString GetServerEndpointMessageAddressId() const = 0;
	
	/** Returns Message Address UID for discovery manager endpoint */
	virtual FString GetDiscoveryManagerMessageAddressId() const = 0;

	/** Returns implementation of StormSync Heartbeat Emitter runnable */
	virtual FStormSyncHeartbeatEmitter& GetHeartbeatEmitter() const = 0;

	/** Returns whether Storm Sync Server endpoint is currently active and running */
	virtual bool IsRunning() const = 0;
	
	/** Returns whether Storm Sync Server endpoint is currently active and running, along with a status text indicating current status and endpoint addresses (message bus and tcp server) */
	virtual bool GetServerStatus(FText& OutStatusText) const = 0;
};
